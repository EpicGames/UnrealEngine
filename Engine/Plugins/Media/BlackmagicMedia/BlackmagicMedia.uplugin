{
	"FileVersion": 3,

	"FriendlyName": "Blackmagic Media Player",
	"Version": 2,
	"VersionName": "2.1",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"MarketplaceURL": "",
	"Description": "Implements input and output using Blackmagic Capture cards.",
	"Category": "Media Players",
	"CanContainContent": false,
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"SupportedTargetPlatforms": [ "Win64", "Linux" ],
	"Modules": [
		{
			"Name": "BlackmagicCore",
			"Type": "Runtime",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "BlackmagicMedia",
			"Type": "Runtime",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "BlackmagicMediaOutput",
			"Type": "Runtime",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "BlackmagicMediaFactory",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "BlackmagicMediaFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "BlackmagicMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		}
	],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		}
	]
}
