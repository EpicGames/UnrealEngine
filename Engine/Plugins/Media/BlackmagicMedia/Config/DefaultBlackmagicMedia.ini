[CoreRedirects]

; 4.22
+EnumRedirects=(OldName="EBlackmagicMediaOutputPixelFormat", ValueChanges=(("PF_8BIT_ARGB", "PF_8BIT_YUV"), ("PF_10BIT_RGB", "PF_10BIT_YUV")))
+EnumRedirects=(OldName="EBlackmagicMediaSourceColorFormat", ValueChanges=(("BGRA", "YUV8"), ("BGR10", "YUV10")))
