// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BlackmagicCore : ModuleRules
{
	public BlackmagicCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlackmagicSDK",
				"MediaIOCore",
				"GPUTextureTransfer"
			});
			
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RHI",
				"Engine"
			});

		// Disabled for initial submission, should be removed after a cleanup and port to code standard unreal code.
		bDisableStaticAnalysis = true;
	}
}



