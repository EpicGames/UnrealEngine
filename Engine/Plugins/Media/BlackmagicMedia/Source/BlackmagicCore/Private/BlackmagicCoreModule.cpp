// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicCoreModule.h"
#include "BlackmagicLib.h"


DEFINE_LOG_CATEGORY(LogBlackmagicCore);

void FBlackmagicCoreModule::StartupModule()
{
	bCanForceBlackmagicUsage = FParse::Param(FCommandLine::Get(), TEXT("forceblackmagicusage"));

	if (CanUseBlackmagicCard())
	{
		bInitialized = BlackmagicDesign::ApiInitialization();
	}
}
void FBlackmagicCoreModule::ShutdownModule()
{
	if (bInitialized)
	{
		bInitialized = false;
		BlackmagicDesign::ApiUninitialization();
	}
}


IMPLEMENT_MODULE(FBlackmagicCoreModule, BlackmagicCore);
