// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicDevice.h"

#include "Common.h"

#include "BlackmagicInputChannel.h"
#include "BlackmagicOutputChannel.h"
#include "BlackmagicHelper.h"


namespace BlackmagicDesign
{
	namespace Private
	{
		FDevice* FDevice::DeviceInstance = nullptr;

		void FDevice::CreateInstance()
		{
			DeviceInstance = new FDevice();
		}

		void FDevice::DestroyInstance()
		{
			delete DeviceInstance;
			DeviceInstance = nullptr;
		}

		void StaticThread_CommandThread(FDevice* Device)
		{
			Device->Thread_CommandThread();
		}

		FDevice::FDevice()
		{
			bIsCommandThreadRunning = true;
			CommandThread = std::thread(StaticThread_CommandThread, this);
		}

		FDevice::~FDevice()
		{
			{
				std::unique_lock<std::mutex> Lock(CommandMutex);
				bIsCommandThreadRunning = false;
				CommandVariable.notify_all();
			}
			CommandThread.join(); // wait for all commands to be process

			check(Commands.size() == 0);
			check(Channels.size() == 0);
			check(OutputChannels.size() == 0);
		}

		void FDevice::AddCommand(FCommand* InCommand)
		{
			std::unique_lock<std::mutex> Lock(CommandMutex);
			if (bIsCommandThreadRunning)
			{
				Commands.push_back(InCommand);
				CommandVariable.notify_all();
			}
		}

		FOutputChannel* FDevice::FindChannel(const FChannelInfo& InChannelInfo)
		{
			FOutputChannel* FoundChannel = nullptr;
			std::lock_guard<std::mutex> Lock(ChannelsMutex);
			auto FoundItt = std::find_if(std::begin(OutputChannels), std::end(OutputChannels), [&](FOutputChannel* Other) { return Other->GetChannelInfo().DeviceIndex == InChannelInfo.DeviceIndex; });
			if (FoundItt != std::end(OutputChannels))
			{
				FoundChannel = *FoundItt;
				check(FoundChannel->bIsDebugAlive);
			}

			return FoundChannel;
		}

		void FDevice::Thread_CommandThread()
		{
			while (bIsCommandThreadRunning)
			{
				// do I have commands if do execute them all
				while (Commands.size() > 0 && bIsCommandThreadRunning)
				{
					FCommand* FrontCommands = nullptr;
					{
						std::unique_lock<std::mutex> Lock(CommandMutex);
						if (Commands.size() > 0)
						{
							FrontCommands = Commands.front();
							Commands.erase(std::begin(Commands));
						}
					}

					if (FrontCommands)
					{
						FrontCommands->Execute();
						delete FrontCommands;
					}
				}

				// Wait for next command
				std::unique_lock<std::mutex> Lock(CommandMutex);
				if (bIsCommandThreadRunning)
				{
					CommandVariable.wait(Lock);
				}
			}

			std::unique_lock<std::mutex> Lock(CommandMutex);
			for (FCommand* Command : Commands)
			{
				if (Command->MustExecuteOnShutdown())
				{
					Command->Execute();
				}

				delete Command;
			}

			Commands.clear();
		}


		///////////////////////////////
		// List of commands
		///////////////////////////////
		FUniqueIdentifier FDevice::RegisterChannel(const FChannelInfo& InChannelInfo, const FInputChannelOptions& InInputChannelOptions, ReferencePtr<IInputEventCallback> InCallback)
		{
			if (InCallback == nullptr)
			{
				return FUniqueIdentifier();
			}

			struct FRegisterCommand : FCommand
			{
				FChannelInfo ChannelInfo;
				FInputChannelOptions InputChannelOptions;
				FUniqueIdentifier UniqueIdentifier;
				ReferencePtr<IInputEventCallback> Callback;
				FDevice* Device;

				virtual void Execute() override
				{
					FInputChannel* FoundChannel = nullptr;
					{
						std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
						auto FoundItt = std::find_if(std::begin(Device->Channels), std::end(Device->Channels), [&](FInputChannel* Other) { return Other->GetChannelInfo() == ChannelInfo; });
						if (FoundItt != std::end(Device->Channels))
						{
							FoundChannel = *FoundItt;
							check(FoundChannel);
						}
					}

					if (FoundChannel)
					{
						if (FoundChannel->IsCompatible(ChannelInfo, InputChannelOptions))
						{
							FoundChannel->InitializeListener(UniqueIdentifier, InputChannelOptions, Callback);
						}
						else
						{
							UE_LOGF(LogBlackmagicCore, Error,"The channel options are not compatible with the already defined options for channel '%d'.", ChannelInfo.DeviceIndex);
							Callback->OnInitializationCompleted(false);
						}
					}
					else
					{
						FInputChannel* NewChannel = new FInputChannel(ChannelInfo);
						if (NewChannel->Initialize(InputChannelOptions))
						{
							if (NewChannel->InitializeListener(UniqueIdentifier, InputChannelOptions, Callback))
							{
								std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
								Device->Channels.push_back(NewChannel);
							}
							else
							{
								delete NewChannel;
								Callback->OnInitializationCompleted(false);
							}
						}
						else
						{
							delete NewChannel;
							Callback->OnInitializationCompleted(false);
						}
					}
				}

				virtual bool MustExecuteOnShutdown() const override 
				{
					return false;
				}
			};

			FRegisterCommand* Command = new FRegisterCommand();
			Command->ChannelInfo = InChannelInfo;
			Command->InputChannelOptions = InInputChannelOptions;
			Command->Callback = std::move(InCallback);
			Command->UniqueIdentifier = UniqueIdentifierGenerator::GenerateNextUniqueIdentifier();
			Command->Device = this;
			AddCommand(Command);
			return Command->UniqueIdentifier;
		}

		void FDevice::UnregisterCallbackForChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier)
		{
			if (!InIdentifier.IsValid())
			{
				return;
			}

			struct FUnregisterCommand : FCommand
			{
				FChannelInfo ChannelInfo;
				FUniqueIdentifier UniqueIdentifier;
				FDevice* Device;

				virtual void Execute() override
				{
					auto FindLambda = [&](FInputChannel* Other) { return Other->GetChannelInfo() == ChannelInfo; };
					FInputChannel* FoundChannel = nullptr;
					{
						std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
						auto FoundItt = std::find_if(std::begin(Device->Channels), std::end(Device->Channels), FindLambda);
						if (FoundItt != std::end(Device->Channels))
						{
							FoundChannel = *FoundItt;
							check(FoundChannel);
						}
					}

					if (FoundChannel)
					{
						FoundChannel->UninitializeListener(UniqueIdentifier);

						if (FoundChannel->NumberOfListeners() == 0)
						{
							{
								std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
								auto FoundItt = std::find_if(std::begin(Device->Channels), std::end(Device->Channels), FindLambda);
								check(FoundItt != std::end(Device->Channels));
								Device->Channels.erase(FoundItt);
							}
							
							delete FoundChannel;
						}
					}
				}

				virtual bool MustExecuteOnShutdown() const override
				{
					return true;
				}
			};

			FUnregisterCommand* Command = new FUnregisterCommand();
			Command->ChannelInfo = InChannelInfo;
			Command->UniqueIdentifier = InIdentifier;
			Command->Device = this;
			AddCommand(Command);
		}

		FUniqueIdentifier FDevice::RegisterOutputChannel(const FChannelInfo& InChannelInfo, const FOutputChannelOptions& InChannelOptions, ReferencePtr<IOutputEventCallback> InCallback)
		{
			if (InCallback == nullptr)
			{
				return FUniqueIdentifier();
			}

			struct FRegisterOutputCommand : FCommand
			{
				FChannelInfo ChannelInfo;
				FOutputChannelOptions ChannelOptions;
				FUniqueIdentifier Identifier;
				ReferencePtr<IOutputEventCallback> Callback;
				FDevice* Device;

				virtual void Execute() override
				{
					FOutputChannel* FoundChannel = nullptr;
					{
						std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
						auto FoundItt = std::find_if(std::begin(Device->OutputChannels), std::end(Device->OutputChannels), [&](FOutputChannel* Other) { return Other->GetChannelInfo().DeviceIndex == ChannelInfo.DeviceIndex; });
						if (FoundItt != std::end(Device->OutputChannels))
						{
							FoundChannel = *FoundItt;
							check(FoundChannel);
						}
					}

					if (FoundChannel)
					{
						if (FoundChannel->IsCompatible(ChannelInfo, ChannelOptions))
						{
							Callback->OnInitializationCompleted(true);
							FoundChannel->AddListener(Identifier, Callback);
						}
						else
						{
							UE_LOGF(LogBlackmagicCore, Error,"The channel options are not compatible with the already defined options for channel '%d'.", ChannelInfo.DeviceIndex);
							Callback->OnInitializationCompleted(false);
						}
					}
					else
					{
						FOutputChannel* NewChannel = new FOutputChannel(ChannelInfo, ChannelOptions);
						if (NewChannel->Initialize())
						{
							NewChannel->bIsDebugAlive = true;
							Callback->OnInitializationCompleted(true);
							NewChannel->AddListener(Identifier, Callback);
							{
								std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
								Device->OutputChannels.push_back(NewChannel);
							}
						}
						else
						{
							delete NewChannel;
							Callback->OnInitializationCompleted(false);
						}
					}
				}

				virtual bool MustExecuteOnShutdown() const override
				{
					return false;
				}
			};

			FRegisterOutputCommand* Command = new FRegisterOutputCommand();
			Command->ChannelInfo = InChannelInfo;
			Command->ChannelOptions = InChannelOptions;
			Command->Callback = std::move(InCallback);
			Command->Identifier = UniqueIdentifierGenerator::GenerateNextUniqueIdentifier();
			Command->Device = this;
			AddCommand(Command);

			return Command->Identifier;
		}

		void FDevice::UnregisterOutputChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier)
		{
			if (!InIdentifier.IsValid())
			{
				return;
			}

			struct FUnregisterOutputCommand : FCommand
			{
				FChannelInfo ChannelInfo;
				FUniqueIdentifier Identifier;
				FDevice* Device;

				virtual void Execute() override
				{
					auto FindLambda = [&](FOutputChannel* Other) { return Other->GetChannelInfo().DeviceIndex == ChannelInfo.DeviceIndex; };

					FOutputChannel* FoundChannel = nullptr;
					{
						std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
						auto FoundItt = std::find_if(std::begin(Device->OutputChannels), std::end(Device->OutputChannels), FindLambda);
						if (FoundItt != std::end(Device->OutputChannels))
						{
							FoundChannel = *FoundItt;
							check(FoundChannel);
						}
					}

					if (FoundChannel)
					{
						FoundChannel->RemoveListener(Identifier);
						if (FoundChannel->NumberOfListeners() == 0)
						{
							FoundChannel->bIsDebugAlive = false;

							{
								std::lock_guard<std::mutex> Lock(Device->ChannelsMutex);
								auto FoundItt = std::find_if(std::begin(Device->OutputChannels), std::end(Device->OutputChannels), FindLambda);
								check(FoundItt != std::end(Device->OutputChannels));
								Device->OutputChannels.erase(FoundItt);
							}
							
							delete FoundChannel;
						}
					}
				}

				virtual bool MustExecuteOnShutdown() const override
				{
					return true;
				}
			};

			if (FOutputChannel* FoundChannel = FindChannel(InChannelInfo))
			{
				FoundChannel->UnregisterBuffers();
			}

			FUnregisterOutputCommand* Command = new FUnregisterOutputCommand();
			Command->ChannelInfo = InChannelInfo;
			Command->Identifier = InIdentifier;
			Command->Device = this;
			AddCommand(Command);
		}

		bool FDevice::SendVideoFrameData(const FChannelInfo& InChannelInfo, const FFrameDescriptor& InFrame)
		{
			// SendVideoFrameData and FUnregisterOutputCommand can't happen at the same time.
			if (FOutputChannel * FoundChannel = FindChannel(InChannelInfo))
			{
				return FoundChannel->SetVideoFrameData(InFrame);
			}

			return false;
		}

		bool FDevice::ShouldCaptureThisFrame(const FChannelInfo& InChannelInfo, ::FTimecode Timecode, uint32 FrameNumber)
		{
			if (FOutputChannel* FoundChannel = FindChannel(InChannelInfo))
			{
				return FoundChannel->ShouldCaptureThisFrame(Timecode, FrameNumber);
			}
			return false;
		}

		bool FDevice::SendVideoFrameData(const FChannelInfo& InChannelInfo, FFrameDescriptor_GPUDMA& InFrame)
		{
			// SendVideoFrameData and FUnregisterOutputCommand can't happen at the same time.
			if (FOutputChannel* FoundChannel = FindChannel(InChannelInfo))
			{
				return FoundChannel->SetVideoFrameData(InFrame);
			}

			return false;
		}


		bool FDevice::SendAudioSamples(const FChannelInfo& InChannelInfo, const FAudioSamplesDescriptor& InSamples)
		{
			// SendAudioSamples and FUnregisterOutputCommand can't happen at the same time.
			if (FOutputChannel* FoundChannel = FindChannel(InChannelInfo))
			{
				return FoundChannel->SetAudioFrameData(InSamples);
			}

			return false;
		}
	}
};
