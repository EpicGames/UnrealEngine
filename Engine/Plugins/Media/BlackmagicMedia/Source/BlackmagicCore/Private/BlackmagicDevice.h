// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlackmagicLib.h"
#include "Common.h"

namespace BlackmagicDesign
{
	namespace Private
	{
		class FInputChannel;
		class FOutputChannel;

		class FDevice
		{
		public:
			static FDevice* GetDevice() { return DeviceInstance; }
			static void CreateInstance();
			static void DestroyInstance();

		public:
			FUniqueIdentifier RegisterChannel(const FChannelInfo& InChannelInfo, const FInputChannelOptions& InInputChannelOptions, ReferencePtr<IInputEventCallback> InCallback);
			void UnregisterCallbackForChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier);
			FUniqueIdentifier RegisterOutputChannel(const FChannelInfo& InChannelInfo, const FOutputChannelOptions& InChannelOptions, ReferencePtr<IOutputEventCallback> InCallback);
			void UnregisterOutputChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier);
			bool SendVideoFrameData(const FChannelInfo& InChannelInfo, const FFrameDescriptor& InFrame);
			bool ShouldCaptureThisFrame(const FChannelInfo& InChannelInfo, ::FTimecode Timecode, uint32 FrameNumber);
			bool SendVideoFrameData(const FChannelInfo& InChannelInfo, FFrameDescriptor_GPUDMA& InFrame);
			bool SendAudioSamples(const FChannelInfo& InChannelInfo, const FAudioSamplesDescriptor& InSamples);
		public:
			struct FCommand
			{
				virtual ~FCommand() {}
				virtual void Execute() = 0;
				virtual bool MustExecuteOnShutdown() const = 0;
			};

			void Thread_CommandThread();

		private:
			void AddCommand(FCommand* InCommand);
			FOutputChannel* FindChannel(const FChannelInfo& InChannelInfo);

		private:
			static FDevice* DeviceInstance;
			FDevice();
			~FDevice();
			FDevice(const FDevice&) = delete;
			FDevice& operator=(const FDevice&) = delete;

		public:
			std::vector<FInputChannel*> Channels;
			std::vector<FOutputChannel*> OutputChannels;
			std::mutex ChannelsMutex;

		private:
			std::condition_variable CommandVariable;
			std::mutex CommandMutex;
			std::thread CommandThread;
			std::vector<FCommand*> Commands;
			volatile bool bIsCommandThreadRunning;
		};
	}
}
