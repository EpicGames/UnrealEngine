// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicDeviceScanner.h"

#include <algorithm>
#include <string>

namespace BlackmagicDesign
{
	namespace Private
	{
		/* DeviceScanner implementation
		*****************************************************************************/
		DeviceScanner::DeviceScanner()
		{
			IDeckLinkIterator* DeckLinkIterator = BlackmagicPlatform::CreateDeckLinkIterator();
			if (DeckLinkIterator)
			{
				IDeckLink* DeckLink = nullptr;
				while (DeckLinkIterator->Next(&DeckLink) == S_OK)
				{
					Devices.push_back(DeckLink);
				}

				BlackmagicPlatform::DestroyDeckLinkIterator(DeckLinkIterator);
			}
		}

		DeviceScanner::~DeviceScanner()
		{
			for (auto Device : Devices)
			{
				if (Device) 
				{ 
					Device->Release(); 
				}
			}
		}

		int32_t DeviceScanner::GetNumDevices() const
		{
			return (int32_t)Devices.size();
		}

		bool DeviceScanner::GetDeviceTextId(int32_t InDeviceIndex, BlackmagicDesign::BlackmagicDeviceScanner::FormatedTextType& OutTextId) const
		{
			IDeckLink* Device = GetDevice(InDeviceIndex);
			if (Device == nullptr)
			{
				return false;
			}

			return BlackmagicPlatform::GetDisplayName(Device, OutTextId, BlackmagicDesign::BlackmagicDeviceScanner::FormatedTextSize);
		}

		bool DeviceScanner::GetDeviceInfo(int32_t InDeviceIndex, BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo& OutDeviceInfo) const
		{
			bool bSuccess = false;
			HRESULT Result = S_FALSE;

			IDeckLink* DeckLink = GetDevice(InDeviceIndex);
			if (DeckLink != nullptr)
			{
				OutDeviceInfo = { 0 };

				IDeckLinkProfileAttributes* Attributes = nullptr;
				Result = DeckLink->QueryInterface(IID_IDeckLinkProfileAttributes, (void**)&Attributes);
				if (Result == S_OK)
				{
					BOOL bSupported = false;
					Result = Attributes->GetFlag(BMDDeckLinkSupportsDualLinkSDI, &bSupported);
					if (Result == S_OK)
					{
						OutDeviceInfo.bCanDoDualLink = bSupported != 0;
					}

					Result = Attributes->GetFlag(BMDDeckLinkSupportsQuadLinkSDI, &bSupported);
					if (Result == S_OK)
					{
						OutDeviceInfo.bCanDoQuadLink = bSupported != 0;
					}

					Result = Attributes->GetFlag(BMDDeckLinkHasReferenceInput, &bSupported);
					if (Result == S_OK)
					{
						OutDeviceInfo.bHasGenlockReferenceInput = bSupported != 0;
					}

					int64_t Connections = 0;
					Result = Attributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkVideoInputConnections, &Connections);
					if (Result == S_OK)
					{
						int64_t VideoCapabilities = 0;
						Result = Attributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkVideoIOSupport, &VideoCapabilities);
						if (Result == S_OK)
						{
							if (((Connections & ENUM(BMDVideoConnection)::bmdVideoConnectionSDI) != 0) || ((Connections & ENUM(BMDVideoConnection)::bmdVideoConnectionHDMI) != 0))
							{
								OutDeviceInfo.bCanDoCapture = (VideoCapabilities & bmdDeviceSupportsCapture) != 0;
								OutDeviceInfo.bCanDoPlayback = (VideoCapabilities & bmdDeviceSupportsPlayback) != 0;
							}
						}
					}

					int64_t SubDeviceCount = 0;
					Result = Attributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkNumberOfSubDevices, &SubDeviceCount);
					if (Result == S_OK)
					{
						OutDeviceInfo.NumberOfSubDevices = (uint32_t)SubDeviceCount;
					}

					int64_t PersistenId = 0;
					Result = Attributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkPersistentID, &PersistenId);
					if (Result == S_OK)
					{
						OutDeviceInfo.DevicePersistentId = (uint32_t)PersistenId;
					}

					int64_t ProfileId = 0;
					Result = Attributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkProfileID, &ProfileId);
					if (Result == S_OK)
					{
						OutDeviceInfo.ProfileId = (uint32_t)ProfileId;
					}

					int64_t GroupPersistenId = 0;
					Result = Attributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkDeviceGroupID, &GroupPersistenId);
					if (Result == S_OK)
					{
						OutDeviceInfo.DeviceGroupId = (uint32_t)GroupPersistenId;
					}

					int64_t SubDeviceIndex = 0;
					Result = Attributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkSubDeviceIndex, &SubDeviceIndex);
					if (Result == S_OK)
					{
						OutDeviceInfo.SubDeviceIndex = (uint32_t)SubDeviceIndex;
					}

					int64_t Duplex = 0;
					Result = Attributes->GetInt(BMDDeckLinkDuplex, &Duplex);
					if (Result == S_OK)
					{
						OutDeviceInfo.bCanDoFullDuplex = Duplex == bmdDuplexFull;
					}

					Result = Attributes->GetFlag(BMDDeckLinkHasLTCTimecodeInput, &bSupported);
					if (Result == S_OK)
					{
						OutDeviceInfo.bHasLTCTimecodeInput = bSupported != 0;
					}

					Result = Attributes->GetFlag(BMDDeckLinkSupportsInputFormatDetection, &bSupported);
					if (Result == S_OK)
					{
						OutDeviceInfo.bCanAutoDetectInputFormat = bSupported != 0;
					}

					Result = Attributes->GetFlag(BMDDeckLinkSupportsInternalKeying, &bSupported);
					if (Result == S_OK)
					{
						OutDeviceInfo.bSupportInternalKeying = bSupported != 0;
					}

					Result = Attributes->GetFlag(BMDDeckLinkSupportsExternalKeying, &bSupported);
					if (Result == S_OK)
					{
						OutDeviceInfo.bSupportExternalKeying = bSupported != 0;
					}

					OutDeviceInfo.bIsSupported = true;

					Attributes->Release();
					bSuccess = true;
				}

				if (bSuccess)
				{
					bSuccess = false;

					// Obtain the configuration interface for the DeckLink device
					IDeckLinkConfiguration* DeckLinkConfiguration = nullptr;
					Result = DeckLink->QueryInterface(IID_IDeckLinkConfiguration, (void**)&DeckLinkConfiguration);
					if (Result == S_OK)
					{
						BOOL bCurrentValue = false;

						// It would return E_NOTIMPL if the feature is not implemented.
						Result = DeckLinkConfiguration->GetFlag(bmdDeckLinkConfigQuadLinkSDIVideoOutputSquareDivisionSplit, &bCurrentValue);
						if (Result == S_OK)
						{
							OutDeviceInfo.bCanDoQuadSquareLink = true;
						}
						bSuccess = true;

						DeckLinkConfiguration->Release();
					}
				}
			}

			return bSuccess;
		}

		IDeckLink* DeviceScanner::GetDevice(int32_t InDeviceIndex) const
		{
			if (InDeviceIndex >= 1 && InDeviceIndex <= Devices.size())
			{
				return Devices[InDeviceIndex-1];
			}

			return nullptr;
		}
	}
}
