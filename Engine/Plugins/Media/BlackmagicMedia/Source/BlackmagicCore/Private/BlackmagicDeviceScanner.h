// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.h"

#include <vector>
#include <memory>

namespace BlackmagicDesign
{
	namespace Private
	{
		/* DeviceScanner definition
		*****************************************************************************/
		class DeviceScanner
		{
		public:
			DeviceScanner();
			~DeviceScanner();

			DeviceScanner(const DeviceScanner&) = delete;
			DeviceScanner& operator=(const DeviceScanner&) = delete;

			int32_t GetNumDevices() const;
			bool GetDeviceTextId(int32_t InDeviceIndex, BlackmagicDesign::BlackmagicDeviceScanner::FormatedTextType& OutTextId) const;
			bool GetDeviceInfo(int32_t InDeviceIndex, BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo& OutDeviceInfo) const;

		private:
			IDeckLink* GetDevice(int32_t InDeviceIndex) const;

		private:
			std::vector<IDeckLink*> Devices;
		};
	}
}
