// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicHelper.h"

#include "Common.h"
#include <chrono>

/*
 * Global helper functions
 */

namespace BlackmagicDesign
{
	namespace Private
	{
		/* UniqueIdentifierGenerator
		*****************************************************************************/
		const int32_t UniqueIdentifierGenerator::InvalidId = 0;
		std::atomic_char32_t UniqueIdentifierGenerator::CurrentId(1);

		FUniqueIdentifier UniqueIdentifierGenerator::GenerateNextUniqueIdentifier()
		{
			int32_t NewId = CurrentId++;
			if (NewId == UniqueIdentifierGenerator::InvalidId)
			{
				NewId = CurrentId++;
			}
			return FUniqueIdentifier(NewId);
		}

		/* Helpers
		*****************************************************************************/
		bool Helpers::EPixelFormatToBMDPixelFormat(EPixelFormat InPixelFormat, BMDPixelFormat& OutPixelFormat)
		{
			switch (InPixelFormat)
			{
			case EPixelFormat::pf_8Bits:	OutPixelFormat = ENUM(BMDPixelFormat)::bmdFormat8BitYUV;	return true;
			case EPixelFormat::pf_10Bits:	OutPixelFormat = ENUM(BMDPixelFormat)::bmdFormat10BitYUV;	return true;
			}
			return false;
		}

		bool Helpers::ETimecodeFormatToBMDTimecodeFormat(ETimecodeFormat InTimecodeFormat, BMDTimecodeFormat& OutTimecodeFormat)
		{
			switch (InTimecodeFormat)
			{
			case ETimecodeFormat::TCF_LTC:		OutTimecodeFormat = ENUM(BMDTimecodeFormat)::bmdTimecodeRP188LTC;		return true;
			case ETimecodeFormat::TCF_VITC1:	OutTimecodeFormat = ENUM(BMDTimecodeFormat)::bmdTimecodeRP188VITC1;		return true;
			case ETimecodeFormat::TCF_Auto:		OutTimecodeFormat = ENUM(BMDTimecodeFormat)::bmdTimecodeRP188Any;		return true;
			case ETimecodeFormat::TCF_None:		return false;
			}
			return false;
		}

		bool Helpers::BMDPixelFormatToEPixelFormat(BMDPixelFormat InPixelFormat, EPixelFormat& OutPixelFormat, EFullPixelFormat& OutFullPixelFormat)
		{
			switch (InPixelFormat)
			{
			case ENUM(BMDPixelFormat)::bmdFormat8BitBGRA:		OutPixelFormat = EPixelFormat::pf_8Bits;	OutFullPixelFormat = EFullPixelFormat::pf_8BitBGRA;		return true;
			case ENUM(BMDPixelFormat)::bmdFormat8BitYUV:		OutPixelFormat = EPixelFormat::pf_8Bits;	OutFullPixelFormat = EFullPixelFormat::pf_8BitYUV;		return true;
			case ENUM(BMDPixelFormat)::bmdFormat10BitRGB:		OutPixelFormat = EPixelFormat::pf_10Bits;	OutFullPixelFormat = EFullPixelFormat::pf_10BitRGB;		return true;
			case ENUM(BMDPixelFormat)::bmdFormat10BitRGBXLE:	OutPixelFormat = EPixelFormat::pf_10Bits;	OutFullPixelFormat = EFullPixelFormat::pf_10BitRGBXLE;	return true;
			case ENUM(BMDPixelFormat)::bmdFormat10BitYUV:		OutPixelFormat = EPixelFormat::pf_10Bits;	OutFullPixelFormat = EFullPixelFormat::pf_10BitYUV;		return true;
			case ENUM(BMDPixelFormat)::bmdFormat10BitRGBX:	OutPixelFormat = EPixelFormat::pf_10Bits;		OutFullPixelFormat = EFullPixelFormat::pf_10BitRGBX;	return true;
			}
			return false;
		}

		bool Helpers::EFieldDominanceToBMDFieldDominance(EFieldDominance InFieldDominance, BMDFieldDominance& OutFieldDominance)
		{
			switch (InFieldDominance)
			{
			case EFieldDominance::Progressive:					OutFieldDominance = ENUM(BMDFieldDominance)::bmdProgressiveFrame;				return true;
			case EFieldDominance::Interlaced:					OutFieldDominance = ENUM(BMDFieldDominance)::bmdLowerFieldFirst;				return true;
			case EFieldDominance::ProgressiveSegmentedFrame:	OutFieldDominance = ENUM(BMDFieldDominance)::bmdProgressiveSegmentedFrame;	return true;
			}
			return false;
		}

		bool Helpers::BMDFieldDominanceToEFieldDominance(BMDFieldDominance InFieldDominance, EFieldDominance& OutFieldDominance)
		{
			switch (InFieldDominance)
			{
			case ENUM(BMDFieldDominance)::bmdLowerFieldFirst:				OutFieldDominance = EFieldDominance::Interlaced;					return true;
			case ENUM(BMDFieldDominance)::bmdUpperFieldFirst:				OutFieldDominance = EFieldDominance::Interlaced;					return true;
			case ENUM(BMDFieldDominance)::bmdProgressiveFrame:			OutFieldDominance = EFieldDominance::Progressive;					return true;
			case ENUM(BMDFieldDominance)::bmdProgressiveSegmentedFrame:	OutFieldDominance = EFieldDominance::ProgressiveSegmentedFrame;		return true;
			}
			return false;
		}

		bool Helpers::Is8Bits(EPixelFormat InPixelFormat)
		{
			return InPixelFormat == EPixelFormat::pf_8Bits;
		}

		bool Helpers::Is8Bits(BMDPixelFormat InPixelFormat)
		{
			return InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat8BitYUV
				|| InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat8BitARGB
				|| InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat8BitBGRA;
		}

		bool Helpers::Is10Bits(EPixelFormat InPixelFormat)
		{
			return InPixelFormat == EPixelFormat::pf_10Bits;
		}

		bool Helpers::Is10Bits(BMDPixelFormat InPixelFormat)
		{
			return InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat10BitYUV
				|| InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat10BitRGB
				|| InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat10BitRGBXLE
				|| InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat10BitRGBX;
		}

		bool Helpers::IsYUV(BMDPixelFormat InPixelFormat)
		{
			return InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat8BitYUV || InPixelFormat == ENUM(BMDPixelFormat)::bmdFormat10BitYUV;
		}

		//TODO ST : Add missing modes
		bool Helpers::IsSDFormat(const BMDDisplayMode InMode)
		{
			return InMode == ENUM(BMDDisplayMode)::bmdModeNTSC
				|| InMode == ENUM(BMDDisplayMode)::bmdModeNTSC2398
				|| InMode == ENUM(BMDDisplayMode)::bmdModePAL
				|| InMode == ENUM(BMDDisplayMode)::bmdModePALp;
		}

		bool Helpers::IsHDFormat(const BMDDisplayMode InMode)
		{
			return InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p2398
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p24
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p25
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p2997
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p30
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p50
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p5994
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080p6000
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080i50
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080i5994
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080i6000
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD720p50
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD720p5994
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD720p60;
		}

		bool Helpers::Is2kFormat(const BMDDisplayMode InMode)
		{
			return InMode == ENUM(BMDDisplayMode)::bmdMode2k2398
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2k24
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2k25
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI2398
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI24
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI25
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI2997
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI30
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI50
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI5994
				|| InMode == ENUM(BMDDisplayMode)::bmdMode2kDCI60;
		}

		bool Helpers::Is4kFormat(const BMDDisplayMode InMode)
		{
			return InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p2398
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p24
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p25
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p2997
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p30
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p50
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p5994
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4K2160p60
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI2398
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI24
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI25
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI2997
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI30
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI50
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI5994
				|| InMode == ENUM(BMDDisplayMode)::bmdMode4kDCI60;
		}

		bool Helpers::Is8kFormat(const BMDDisplayMode InMode)
		{
			return InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p2398
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p24
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p25
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p2997
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p30
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p50
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p5994
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8K4320p60
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI2398
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI24
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI25
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI2997
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI30
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI50
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI5994
				|| InMode == ENUM(BMDDisplayMode)::bmdMode8kDCI60;
		}

		bool Helpers::IsInterlacedDisplayMode(const BMDDisplayMode InMode)
		{
			return InMode == ENUM(BMDDisplayMode)::bmdModeNTSC
				|| InMode == ENUM(BMDDisplayMode)::bmdModePAL
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080i50
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080i5994
				|| InMode == ENUM(BMDDisplayMode)::bmdModeHD1080i6000;
		}

		void Helpers::BlockForAmountOfTime(float InSeconds)
		{
			std::chrono::nanoseconds TimeToWait = std::chrono::nanoseconds(int64_t(InSeconds * 1000.0f * 1000.0f * 1000.0f));
			const std::chrono::nanoseconds WhileThreshold = std::chrono::nanoseconds(100 * 1000); //100us
			const std::chrono::nanoseconds SleepZeroThreshold = std::chrono::nanoseconds(5 * 1000 * 1000); //5ms

			if (TimeToWait < std::chrono::nanoseconds(100))
			{
				return;
			}

			if (TimeToWait > SleepZeroThreshold)
			{
				const std::chrono::nanoseconds SleepFor = TimeToWait - SleepZeroThreshold;
				std::this_thread::sleep_for(SleepFor);
				TimeToWait -= SleepFor;
			}

			//Sleep(0) until 100us left
			if (TimeToWait > WhileThreshold)
			{
				const std::chrono::nanoseconds SleepFor = TimeToWait - WhileThreshold;
				std::chrono::high_resolution_clock::time_point TargetTime = std::chrono::high_resolution_clock::now() + (SleepFor);
				while (std::chrono::high_resolution_clock::now() < TargetTime)
				{
					std::this_thread::sleep_for(std::chrono::seconds(0));
				}

				TimeToWait -= SleepFor;
			}

			//While for the last 100us
			const std::chrono::high_resolution_clock::time_point TargetTime = std::chrono::high_resolution_clock::now() + TimeToWait;
			while (std::chrono::high_resolution_clock::now() < TargetTime);
		}

		IDeckLink* Helpers::GetDeviceForIndex(int32_t InIndex)
		{
			IDeckLink* FoundDevice = nullptr;

			// Create an IDeckLinkIterator object to enumerate all DeckLink cards in the system
			IDeckLinkIterator* DeckLinkIterator = BlackmagicPlatform::CreateDeckLinkIterator();

			if (DeckLinkIterator)
			{
				// Obtain the required DeckLink device
				IDeckLink* DeckLinkItt = nullptr;
				int32_t Index = 1;
				for (; (DeckLinkIterator->Next(&DeckLinkItt) == S_OK); ++Index)
				{
					if (Index == InIndex)
					{
						FoundDevice = DeckLinkItt;
						break;
					}
					else
					{
						DeckLinkItt->Release();
					}
				}

				BlackmagicPlatform::DestroyDeckLinkIterator(DeckLinkIterator);
			}

			return FoundDevice;
		}

		IDeckLink* Helpers::GetDeviceForIdentifier(int64_t InIdentifier)
		{
			IDeckLink* FoundDevice = nullptr;

			// Create an IDeckLinkIterator object to enumerate all DeckLink cards in the system
			IDeckLinkIterator* DeckLinkIterator = BlackmagicPlatform::CreateDeckLinkIterator();

			if (DeckLinkIterator)
			{
				// Obtain the required DeckLink device
				IDeckLink* DeckLinkItt = nullptr;
				int32_t Index = 1;
				for (; (DeckLinkIterator->Next(&DeckLinkItt) == S_OK); ++Index)
				{
					ReferencePtr<IDeckLinkProfileAttributes> DeckLinkAttributes;
					HRESULT Result = DeckLinkItt->QueryInterface(IID_IDeckLinkProfileAttributes, (void**)&DeckLinkAttributes);
					if (Result != S_OK)
					{
						UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkAttributes interface looking for device identifier '%lld' - result = %08x.", InIdentifier, Result);
						break;
					}

					int64_t PersistenId = 0;
					DeckLinkAttributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkPersistentID, &PersistenId);
					if (PersistenId == InIdentifier)
					{
						FoundDevice = DeckLinkItt;
						break;
					}
					else
					{
						DeckLinkItt->Release();
					}
				}

				BlackmagicPlatform::DestroyDeckLinkIterator(DeckLinkIterator);
			}

			return FoundDevice;
		}

		FTimecode Helpers::AdjustTimecodeForUE(const FFormatInfo& InFormatInfo, const FTimecode& InTimecode, const FTimecode& InPreviousTimecode)
		{
			FTimecode AdjustedTimecode = InTimecode;

			//Frame number will be capped at 29. So manual adjustment is needed for bigger frame rates or interlaced formats.
			const int32_t FrameRate = InFormatInfo.FrameRateNumerator / InFormatInfo.FrameRateDenominator;
			if (FrameRate > 30 || InFormatInfo.FieldDominance == EFieldDominance::Interlaced)
			{
				AdjustedTimecode.Frames *= 2;
			}

			if ((InPreviousTimecode == InTimecode))
			{
				++AdjustedTimecode.Frames;
			}

			return AdjustedTimecode;
		}

		BlackmagicDesign::FTimecode Helpers::AdjustTimecodeFromUE(const FFormatInfo& InFormatInfo, const FTimecode& InTimecode)
		{
			FTimecode AdjustedTimecode = InTimecode;
			const int32_t FrameRate = InFormatInfo.FrameRateNumerator / InFormatInfo.FrameRateDenominator;
			if (FrameRate > 30 || InFormatInfo.FieldDominance == EFieldDominance::Interlaced)
			{
				AdjustedTimecode.Frames /= 2;
			}

			return AdjustedTimecode;
		}

	}
};
