// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Common.h"
#include "BlackmagicLib.h"

#include <atomic>

namespace BlackmagicDesign
{

	namespace Private
	{
		struct UniqueIdentifierGenerator
		{
		public:
			static const int32_t InvalidId;
			static FUniqueIdentifier GenerateNextUniqueIdentifier();
		private:
			static std::atomic_char32_t CurrentId;
		};

		struct Helpers
		{
		public:
			static bool EPixelFormatToBMDPixelFormat(EPixelFormat InPixelFormat, BMDPixelFormat& OutPixelFormat);
			static bool ETimecodeFormatToBMDTimecodeFormat(ETimecodeFormat InTimecodeFormat, BMDTimecodeFormat& OutTimecodeFormat);
			static bool BMDPixelFormatToEPixelFormat(BMDPixelFormat InPixelFormat, EPixelFormat& OutPixelFormat, EFullPixelFormat& OutFullPixelFormat);
			static bool EFieldDominanceToBMDFieldDominance(EFieldDominance InFieldDominance, BMDFieldDominance& OutFieldDominance);
			static bool BMDFieldDominanceToEFieldDominance(BMDFieldDominance InFieldDominance, EFieldDominance & OutFieldDominance);
			static bool Is8Bits(BMDPixelFormat InPixelFormat);
			static bool Is8Bits(EPixelFormat InPixelFormat);
			static bool Is10Bits(BMDPixelFormat InPixelFormat);
			static bool Is10Bits(EPixelFormat InPixelFormat);
			static bool IsYUV(BMDPixelFormat InPixelFormat);
			static bool IsSDFormat(const BMDDisplayMode InMode);
			static bool IsHDFormat(const BMDDisplayMode InMode);
			static bool Is2kFormat(const BMDDisplayMode InMode);
			static bool Is4kFormat(const BMDDisplayMode InMode);
			static bool Is8kFormat(const BMDDisplayMode InMode);
			static bool IsInterlacedDisplayMode(const BMDDisplayMode InMode);
			static void BlockForAmountOfTime(float InSeconds);

			//Returned DeckLink pointer must be released by the caller
			static IDeckLink* GetDeviceForIndex(int32_t InIndex);
			static IDeckLink* GetDeviceForIdentifier(int64_t InIdentifier);

			static FTimecode AdjustTimecodeForUE(const FFormatInfo& InFormatInfo, const FTimecode& InTimecode, const FTimecode& InPreviousTimecode);
			static FTimecode AdjustTimecodeFromUE(const FFormatInfo& InFormatInfo, const FTimecode& InTimecode);
		};


		struct UE_10Bit_PixelFormat { int32_t R : 10; int32_t G : 10; int32_t B : 10; int32_t A : 2; };
		struct bmd_10Bit_PixelFormat { int32_t Rh : 6; int32_t A : 2; int32_t Gh : 4; int32_t Rl : 4; int32_t Bh : 2; int32_t Gl : 6; int32_t Bl : 8; };
		union Conversion_UE_10Bit_to_bmd_10Bit
		{
			struct { int32_t bmdBl : 8; int32_t bmdBh : 2; };
			struct { int32_t bmdGl : 6; int32_t bmdGh : 4; };
			struct { int32_t bmdRl : 4; int32_t bmdRh : 6; };
			int32_t UEColor : 10;
		};


		static_assert(sizeof(int32_t) == sizeof(UE_10Bit_PixelFormat), "UE_10Bit_PixelFormat is not 32bits.");
		static_assert(sizeof(int32_t) == sizeof(bmd_10Bit_PixelFormat), "bmd_10Bit_PixelFormat is not 32bits.");
		static_assert(sizeof(int32_t) == sizeof(Conversion_UE_10Bit_to_bmd_10Bit), "Conversion_UE_10Bit_to_bmd_10Bit is not 32bits.");
	}
};