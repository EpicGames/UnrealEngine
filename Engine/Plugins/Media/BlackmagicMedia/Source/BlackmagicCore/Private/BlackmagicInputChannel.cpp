// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicInputChannel.h"

#include "Common.h"

#include "BlackmagicHelper.h"
#include "HAL/IConsoleManager.h"

static TAutoConsoleVariable<bool> CVarBlackmagicAlwaysLockToFieldRate(
	TEXT("Blackmagic.Genlock.AlwaysLockToFieldRate"),
	true,
	TEXT("When true, the synthetic odd-field event is fired on every received frame regardless of whether the SDK reports an interlaced display mode. This allows picking a non input source and still getting the desired frame rate (e.g. 60Hz for an interlaced source instead of 30Hz). When false, the event is only fired when the source is interlaced."),
	ECVF_Default);

// Do not use all the available thread. Keep 4 for the 4 Unreal Engine's virtual thread.
const int32_t NumberOfThread = std::thread::hardware_concurrency()-4;

namespace BlackmagicDesign
{
	namespace Private
	{
		enum ResultError
		{
			RS_VideoInputFormatChanged_GetFrameRate = 0x8E100001L,
			RS_VideoInputFormatChanged_BMDFieldDominanceToEFieldDominance = 0x8E100002L,
			RS_VideoInputFrameArrived_GetComponents = 0x8E100003L,
			RS_VideoInputFrameArrived_VideoConvertFrame = 0x8E100004L,
			RS_VideoInputFrameArrived_ConvertFrame = 0x8E100005L,
			RS_VideoInputFrameArrived_BMDPixelFormatToEPixelFormat = 0x8E100006L,
			RS_VideoInputFrameArrived_AudioGetBytes = 0x8E100007L,
			RS_VideoInputFrameArrived_StatusGetFlags = 0x8E100008L,
		};

		const BMDAudioSampleRate bmdDefaultAudioSampleRate = bmdAudioSampleRate48kHz;
		const uint32_t DefaultAudioSampleRate = static_cast<uint32_t>(bmdDefaultAudioSampleRate);
		const BMDAudioSampleType DefaultAudioSampleType = bmdAudioSampleType32bitInteger;

		/** Implements buffer guarding */
		struct FSampleBufferHolder : public BlackmagicDesign::IInputEventCallback::FFrameBufferHolder
		{
		public:
			FSampleBufferHolder(IUnknown* InObject)
				: Object(InObject)
			{
				if (Object)
				{
					Object->AddRef();
				}
			}

			~FSampleBufferHolder()
			{
				if (Object)
				{
					Object->Release();
				}
			}

		private:
			IUnknown* Object = nullptr;
		};


		FInputChannelNotificationCallback::FInputChannelNotificationCallback(FInputChannel* InOwner, IDeckLinkInput* InDeckLinkInput, IDeckLinkStatus* InDeckLinkStatus)
			: InputChannel(InOwner)
			, DeckLinkInput(InDeckLinkInput)
			, DeckLinkStatus(InDeckLinkStatus)
			, RefCount(1)
			, FrameNumber(0)
			, HDRLogCount(0)
		{
		}

		HRESULT FInputChannelNotificationCallback::QueryInterface(REFIID iid, LPVOID *ppv)
		{
			return E_NOINTERFACE;
		}

		ULONG FInputChannelNotificationCallback::AddRef()
		{
			return ++RefCount;
		}

		ULONG FInputChannelNotificationCallback::Release()
		{
			--RefCount;
			if (RefCount == 0)
			{
				delete this;
			}
			return RefCount;
		}

		HRESULT FInputChannelNotificationCallback::VideoInputFormatChanged(BMDVideoInputFormatChangedEvents InNotificationEvents, IDeckLinkDisplayMode* InNewDisplayMode, BMDDetectedVideoInputFormatFlags InDetectedSignalFlags)
		{
			check(InputChannel);

			bool bProcessFormatChanged = false;
			if ((InNotificationEvents & bmdVideoInputDisplayModeChanged) != 0)
			{
				//if display mode hasn't really changed, don't trigger a callback for it
				if (InNewDisplayMode->GetDisplayMode() != InputChannel->DisplayMode)
				{
					std::string DisplayModeName = BlackmagicPlatform::GetName(InNewDisplayMode);
					if (DisplayModeName.empty())
					{
						static std::string InvalidName = "<invalid name>";
						DisplayModeName = InvalidName;
					}

					UE_LOGF(LogBlackmagicCore, Display,"The video input format changed on the device '%d'. The new display mode is '%s'", InputChannel->ChannelInfo.DeviceIndex, DisplayModeName.c_str());
					bProcessFormatChanged = true;
				}
			}

			if ((InNotificationEvents & bmdVideoInputColorspaceChanged) != 0)
			{
				//Verify what is the 'new' color space. This gets called even if it's what was requested since 11.6
				//Since we only support YUV input, if it's not in the signal, trigger a format changed
				if ((InDetectedSignalFlags & bmdDetectedVideoInputYCbCr422) == 0)
				{
					bProcessFormatChanged = true;

					//Color space did change to something not supported / expected
					if (bHasProcessedVideoInput)
					{
						UE_LOGF(LogBlackmagicCore, Display,"The color space have changed on the device '%d'. Only YUV is supported.", InputChannel->ChannelInfo.DeviceIndex);

						if (InDetectedSignalFlags == bmdDetectedVideoInputYCbCr422)
						{
							UE_LOGF(LogBlackmagicCore, Display,"VideoInputYCbCr422");
						}
						if (InDetectedSignalFlags == bmdDetectedVideoInputRGB444)
						{
							UE_LOGF(LogBlackmagicCore, Display,"VideoInputRGB444");
						}
					}
				}
			}
			
			// Always derive the field dominance from the new display mode, regardless of which notification flags
			// were set. The Blackmagic driver can report a display mode change without also flagging a field
			// dominance change (e.g. switching from 1080p30 to 1080i60).
			EFieldDominance NewFieldDominance = EFieldDominance::Progressive;
			if (!Helpers::BMDFieldDominanceToEFieldDominance(InNewDisplayMode->GetFieldDominance(), NewFieldDominance))
			{
				return RS_VideoInputFormatChanged_BMDFieldDominanceToEFieldDominance;
			}

			if ((InNotificationEvents & bmdVideoInputFieldDominanceChanged) != 0)
			{
				//For PSF inputs, the user will select progressive so if the detected signal is PSF, ensure the selection was progressive
				if (NewFieldDominance != InputChannel->ChannelOptions.FormatInfo.FieldDominance
					&& (NewFieldDominance == EFieldDominance::ProgressiveSegmentedFrame && InputChannel->ChannelOptions.FormatInfo.FieldDominance != EFieldDominance::Progressive))
				{
					UE_LOGF(LogBlackmagicCore, Display,"The FieldDominance have changed on the device '%d'.", InputChannel->ChannelInfo.DeviceIndex);
					bProcessFormatChanged = true;
				}
			}

			if (bProcessFormatChanged)
			{
				FFormatInfo FormatInfo;
				FormatInfo.DisplayMode = InNewDisplayMode->GetDisplayMode();

				// Note. The documentation and what is actually received from the function are inversed.
				BMDTimeValue Denominator;
				BMDTimeScale Numerator;
				HRESULT Result = InNewDisplayMode->GetFrameRate(&Denominator, &Numerator);
				if (Result != S_OK || Denominator == 0)
				{
					return RS_VideoInputFormatChanged_GetFrameRate;
				}

				FormatInfo.FrameRateNumerator = (uint32_t)Numerator;
				FormatInfo.FrameRateDenominator = (uint32_t)Denominator;

				FormatInfo.Width = InNewDisplayMode->GetWidth();
				FormatInfo.Height = InNewDisplayMode->GetHeight();
				FormatInfo.FieldDominance = NewFieldDominance;

				//Save the new frame format data in our internal options in case we're still using that channel
				InputChannel->ChannelOptions.FormatInfo.FrameRateNumerator = (uint32_t)Numerator;
				InputChannel->ChannelOptions.FormatInfo.FrameRateDenominator = (uint32_t)Denominator;
				InputChannel->ChannelOptions.FormatInfo.Width = FormatInfo.Width;
				InputChannel->ChannelOptions.FormatInfo.Height = FormatInfo.Height;
				InputChannel->ChannelOptions.FormatInfo.FieldDominance = FormatInfo.FieldDominance;
				InputChannel->ChannelOptions.FormatInfo.DisplayMode = InNewDisplayMode->GetDisplayMode();

				if (InputChannel->ChannelOptions.bAutoDetect || InputChannel->ChannelOptions.TimecodeFormat == ETimecodeFormat::TCF_Auto)
				{
					InputChannel->DisplayMode = InNewDisplayMode->GetDisplayMode();
					InputChannel->Reinitialize();
				}

				std::lock_guard<std::mutex> Lock(InputChannel->ListenerMutex);
				for (FInputChannel::FListener& Listener : InputChannel->Listeners)
				{
					Listener.Option.FormatInfo = InputChannel->ChannelOptions.FormatInfo;
					Listener.Callback->OnFrameFormatChanged(FormatInfo);
				}
			}

			HDRLogCount = 0;

			return S_OK;
		}

		HRESULT FInputChannelNotificationCallback::VideoInputFrameArrived(IDeckLinkVideoInputFrame* InVideoFrame, IDeckLinkAudioInputPacket* InAudioPacket)
		{
			check(InputChannel);

			IInputEventCallback::FFrameReceivedInfo FrameInfo;

			if (InVideoFrame)
			{
				++FrameNumber;
				FrameInfo.FrameNumber = FrameNumber;

				FrameInfo.bHasInputSource = (InVideoFrame->GetFlags() & bmdFrameHasNoInputSource) == 0;

				IDeckLinkTimecode* DeckLinkTimecode = nullptr;
				if (InputChannel->ReadTimecodeCounter > 0)
				{
					HRESULT Result = S_FALSE;
					Result = InVideoFrame->GetTimecode(InputChannel->DeviceTimecodeFormat, &DeckLinkTimecode);
					if (Result == S_FALSE && InputChannel->DeviceTimecodeFormat == ENUM(BMDTimecodeFormat)::bmdTimecodeRP188VITC1)
					{
						//When frame rates are above 30 and VITC timecode is used, the timecode will be valid in alternance on vitc1 and 2.
						Result = InVideoFrame->GetTimecode(ENUM(BMDTimecodeFormat)::bmdTimecodeRP188VITC2, &DeckLinkTimecode);
					}

					FrameInfo.bHaveTimecode = (Result == S_OK && DeckLinkTimecode);
					if (FrameInfo.bHaveTimecode)
					{
						unsigned char Hours, Minutes, Seconds, Frames;
						if (DeckLinkTimecode->GetComponents(&Hours, &Minutes, &Seconds, &Frames) != S_OK)
						{
							return RS_VideoInputFrameArrived_GetComponents;
						}

						FTimecode ReadTimecode;
						ReadTimecode.Hours = Hours;
						ReadTimecode.Minutes = Minutes;
						ReadTimecode.Seconds = Seconds;
						ReadTimecode.Frames = Frames;
						ReadTimecode.bIsDropFrame = (DeckLinkTimecode->GetFlags() & bmdTimecodeIsDropFrame) == bmdTimecodeIsDropFrame;
						DeckLinkTimecode->Release();

						FrameInfo.Timecode = Helpers::AdjustTimecodeForUE(InputChannel->ChannelOptions.FormatInfo, ReadTimecode, InputChannel->PreviousReceivedTimecode);
						InputChannel->PreviousReceivedTimecode = ReadTimecode;
					}
				}

				GetHDRMetaData(FrameInfo, InVideoFrame);
			}

			if (FrameInfo.bHasInputSource && InputChannel->ReadVideoCounter > 0)
			{
				bHasProcessedVideoInput = true;

				check(InVideoFrame);

				IDeckLinkVideoFrame* VideoFrame = InVideoFrame;

				if (!Helpers::BMDPixelFormatToEPixelFormat(VideoFrame->GetPixelFormat(),
					FrameInfo.PixelFormat, FrameInfo.FullPixelFormat))
				{
					return RS_VideoInputFrameArrived_BMDPixelFormatToEPixelFormat;
				}

				FrameInfo.VideoPitch = VideoFrame->GetRowBytes();
				FrameInfo.VideoWidth = VideoFrame->GetWidth();
				FrameInfo.VideoHeight = VideoFrame->GetHeight();
				if (VideoFrame->GetBytes(&FrameInfo.VideoBuffer) != S_OK)
				{
					return RS_VideoInputFrameArrived_VideoConvertFrame;
				}

				FrameInfo.FieldDominance = EFieldDominance::Progressive;
				if ((InVideoFrame->GetFlags() & bmdFrameCapturedAsPsF) != 0)
				{
					FrameInfo.FieldDominance = EFieldDominance::ProgressiveSegmentedFrame;
				}
				else if (Helpers::IsInterlacedDisplayMode(InputChannel->DisplayMode))
				{
					FrameInfo.FieldDominance = EFieldDominance::Interlaced;
				}
			}
			else
			{
				bHasProcessedVideoInput = false;
			}

			//{
			//	IDeckLinkVideoFrameAncillary* DeckLinkAncillary = nullptr;
			//	HRESULT Error = InVideoFrame->GetAncillaryData(&DeckLinkAncillary);
			//	if (Error == S_OK && DeckLinkAncillary)
			//	{
			//		DeckLinkAncillary->GetBufferForVerticalBlankingLine()
			//		DeckLinkAncillary->Release();
			//	}
			//}

			if (InAudioPacket && InputChannel->ReadAudioCounter > 0)
			{
				if (InAudioPacket->GetBytes(&FrameInfo.AudioBuffer) != S_OK)
				{
					return RS_VideoInputFrameArrived_AudioGetBytes;
				}

				check(DefaultAudioSampleType == bmdAudioSampleType32bitInteger);
				const uint32_t SizeOfSample = sizeof(int32_t);
				FrameInfo.AudioBufferSize = InAudioPacket->GetSampleFrameCount() * InputChannel->ChannelOptions.NumberOfAudioChannel * SizeOfSample;
				FrameInfo.NumberOfAudioChannel = InputChannel->ChannelOptions.NumberOfAudioChannel;
				FrameInfo.AudioRate = DefaultAudioSampleRate;
			}

			{
				std::lock_guard<std::mutex> Lock(InputChannel->ListenerMutex);
				for (FInputChannel::FListener& Listener : InputChannel->Listeners)
				{
					Listener.Callback->OnFrameReceived(FrameInfo);

					{
						IInputEventCallback::FFrameReceivedBufferHolders BufferHolders;
						Listener.Callback->OnFrameReceived(FrameInfo, BufferHolders);

						// Instantiate audio buffer holder if requested
						if (BufferHolders.AudioBufferHolder)
						{
							*BufferHolders.AudioBufferHolder = MakeShared<BlackmagicDesign::Private::FSampleBufferHolder>(InAudioPacket);
						}

						// Instantiate video buffer holder if requested
						if (BufferHolders.VideoBufferHolder)
						{
							*BufferHolders.VideoBufferHolder = MakeShared<BlackmagicDesign::Private::FSampleBufferHolder>(InVideoFrame);
						}
					}
				}
			}

			if (Helpers::IsInterlacedDisplayMode(InputChannel->DisplayMode) && (FrameInfo.bHasInputSource || CVarBlackmagicAlwaysLockToFieldRate.GetValueOnAnyThread()))
			{
				if (!InputChannel->bIsWaitingForInterlaceEvent)
				{
					//Wait for artificial field event
					InputChannel->bIsWaitingForInterlaceEvent = true;
					const float SecondsToWait = ((float)InputChannel->ChannelOptions.FormatInfo.FrameRateDenominator / (float)InputChannel->ChannelOptions.FormatInfo.FrameRateNumerator) * 0.5f;
					Helpers::BlockForAmountOfTime(SecondsToWait);
					FrameNumber++;

					{
						std::lock_guard<std::mutex> Lock(InputChannel->ListenerMutex);
						for (FInputChannel::FListener& Listener : InputChannel->Listeners)
						{
							Listener.Callback->OnInterlacedOddFieldEvent(FrameNumber);
						}
					}

					InputChannel->bIsWaitingForInterlaceEvent = false;
				}
				else
				{
					UE_LOGF(LogBlackmagicCore, Warning,"Video frame has arrived faster than expected on device '%d'", InputChannel->ChannelInfo.DeviceIndex);
				}
			}

			return S_OK;
		}

		void FInputChannelNotificationCallback::GetHDRMetaData(IInputEventCallback::FFrameReceivedInfo& FrameInfo, IDeckLinkVideoInputFrame* InVideoFrame)
		{
			FrameInfo.HDRMetaData.bIsAvailable = (InVideoFrame->GetFlags() & bmdFrameContainsHDRMetadata) != 0;
			if ((FrameInfo.HDRMetaData.bIsAvailable) && (FrameInfo.bHasInputSource))
			{
				IDeckLinkVideoFrameMetadataExtensions* MetadataExtensions = nullptr;
				HRESULT Result = InVideoFrame->QueryInterface(
					IID_IDeckLinkVideoFrameMetadataExtensions,
					(void**)&MetadataExtensions);
				if (Result == S_OK)
				{
					int64_t EOTF = 0;
					if (MetadataExtensions->GetInt(
						bmdDeckLinkFrameMetadataHDRElectroOpticalTransferFunc, &EOTF) == S_OK)
					{
						FrameInfo.HDRMetaData.EOTF = (EHDRMetaDataEOTF)EOTF;
					}
					else if (IsHDRLoggingOK())
					{
						UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR EOTF for device '%d'.",
							InputChannel->ChannelInfo.DeviceIndex);
					}

					int64_t Colourspace = 0;
					if (MetadataExtensions->GetInt(
						bmdDeckLinkFrameMetadataColorspace, &Colourspace) == S_OK)
					{
						switch (Colourspace)
						{
						case bmdColorspaceRec601:
							FrameInfo.HDRMetaData.ColorSpace = EHDRMetaDataColorspace::Rec601;
							break;
						case bmdColorspaceRec709:
							FrameInfo.HDRMetaData.ColorSpace = EHDRMetaDataColorspace::Rec709;
							break;
						case bmdColorspaceRec2020:
							FrameInfo.HDRMetaData.ColorSpace = EHDRMetaDataColorspace::Rec2020;
							break;
						}
					}
					else if (IsHDRLoggingOK())
					{
						UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR colourspace for device '%d'.",
							InputChannel->ChannelInfo.DeviceIndex);
					}

					// These fields are populated for PQ streams, not for HLG signals.
					if (FrameInfo.HDRMetaData.EOTF == EHDRMetaDataEOTF::PQ)
					{
						FrameInfo.HDRMetaData.WhitePointX = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRWhitePointX);
						FrameInfo.HDRMetaData.WhitePointY = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRWhitePointY);

						FrameInfo.HDRMetaData.DisplayPrimariesRedX = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRDisplayPrimariesRedX);
						FrameInfo.HDRMetaData.DisplayPrimariesRedY = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRDisplayPrimariesRedY);
						FrameInfo.HDRMetaData.DisplayPrimariesGreenX = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRDisplayPrimariesGreenX);
						FrameInfo.HDRMetaData.DisplayPrimariesGreenY = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRDisplayPrimariesGreenY);
						FrameInfo.HDRMetaData.DisplayPrimariesBlueX = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRDisplayPrimariesBlueX);
						FrameInfo.HDRMetaData.DisplayPrimariesBlueY = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRDisplayPrimariesBlueY);
						FrameInfo.HDRMetaData.MaxDisplayLuminance = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRMaxDisplayMasteringLuminance);
						FrameInfo.HDRMetaData.MinDisplayLuminance = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRMinDisplayMasteringLuminance);
						FrameInfo.HDRMetaData.MaxContentLightLevel = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRMaximumContentLightLevel);
						FrameInfo.HDRMetaData.MaxFrameAverageLightLevel = GetFloatMetaData(MetadataExtensions,
							bmdDeckLinkFrameMetadataHDRMaximumFrameAverageLightLevel);
					}

					MetadataExtensions->Release();
				}
			}
		}

		double FInputChannelNotificationCallback::GetFloatMetaData(IDeckLinkVideoFrameMetadataExtensions* MetadataExtensions, BMDDeckLinkFrameMetadataID MetaDataID)
		{
			double MetaDataValue = 0.0f;

			if (MetadataExtensions->GetFloat(MetaDataID, &MetaDataValue) != S_OK)
			{
				if (IsHDRLoggingOK())
				{
					switch (MetaDataID)
					{
						case bmdDeckLinkFrameMetadataHDRDisplayPrimariesRedX:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR display primaries red X.");
							break;
						case bmdDeckLinkFrameMetadataHDRDisplayPrimariesRedY:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR display primaries red Y.");
							break;
						case bmdDeckLinkFrameMetadataHDRDisplayPrimariesGreenX:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR display primaries green X.");
							break;
						case bmdDeckLinkFrameMetadataHDRDisplayPrimariesGreenY:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR display primaries green Y.");
							break;
						case bmdDeckLinkFrameMetadataHDRDisplayPrimariesBlueX:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR display primaries blue X.");
							break;
						case bmdDeckLinkFrameMetadataHDRDisplayPrimariesBlueY:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR display primaries blue Y.");
							break;
						case bmdDeckLinkFrameMetadataHDRWhitePointX:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR white point X.");
							break;
						case bmdDeckLinkFrameMetadataHDRWhitePointY:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR white point Y.");
							break;
						case bmdDeckLinkFrameMetadataHDRMaxDisplayMasteringLuminance:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR maximum mastering luminance.");
							break;
						case bmdDeckLinkFrameMetadataHDRMinDisplayMasteringLuminance:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR minimum mastering luminance.");
							break;
						case bmdDeckLinkFrameMetadataHDRMaximumContentLightLevel:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR maximum content light level.");
							break;
						case bmdDeckLinkFrameMetadataHDRMaximumFrameAverageLightLevel:
							UE_LOGF(LogBlackmagicCore, Warning,"Could not get meta data HDR maximum frame average light level.");
							break;
					}
				}
			}

			return MetaDataValue;
		}

		bool FInputChannelNotificationCallback::IsHDRLoggingOK()
		{
			// Reset log count after a few seconds.
			if (std::chrono::system_clock::now() > HDRLogResetCountTime)
			{
				HDRLogCount = 0;
			}
			
			bool bIsOK = HDRLogCount < 30;

			if (bIsOK)
			{
				HDRLogCount++;
				HDRLogResetCountTime = std::chrono::system_clock::now() + std::chrono::seconds(2);
			}

			return bIsOK;
		}
		

		FInputChannel::FInputChannel(const FChannelInfo& InChannelInfo)
			: ChannelInfo(InChannelInfo)
			, ReadAudioCounter(0)
			, ReadLTCCounter(0)
			, ReadVideoCounter(0)
			, ReadTimecodeCounter(0)
			, DeckLink(nullptr)
			, DeckLinkInput(nullptr)
			, DeckLinkAttributes(nullptr)
			, DeckLinkConfiguration(nullptr)
			, NotificationCallback(nullptr)
			, DeckLinkStatus(nullptr)
			, bStreamStarted(false)
			, VideoInputFlags(bmdVideoInputFlagDefault)
			, PixelFormat(bmdFormat8BitYUV)
			, DisplayMode(bmdModeHD1080p24)
			, DeviceTimecodeFormat(ENUM(BMDTimecodeFormat)::bmdTimecodeRP188Any)
			, bIsWaitingForInterlaceEvent(false)
			, bCanProcessOddFrame(false)
		{
		}

		bool FInputChannel::IsCompatible(const FChannelInfo& InChannelInfo, const FInputChannelOptions& InChannelOptions) const
		{
			// Format isn't set on the channel options when using "Auto", since we don't care about the format, it's always going to be "compatible".
			bool bResult = (ChannelOptions.FormatInfo.DisplayMode == InChannelOptions.FormatInfo.DisplayMode || InChannelOptions.bAutoDetect);

			if (bResult)
			{
				if (ChannelOptions.bReadAudio && InChannelOptions.bReadAudio)
				{
					bResult = ChannelOptions.NumberOfAudioChannel == InChannelOptions.NumberOfAudioChannel;
				}
			}

			if (bResult)
			{
				if (ChannelOptions.bReadVideo && InChannelOptions.bReadVideo)
				{
					bResult = ChannelOptions.PixelFormat == InChannelOptions.PixelFormat;
				}
			}

			if (bResult)
			{
				if (ChannelOptions.bUseTheDedicatedLTCInput && InChannelOptions.bUseTheDedicatedLTCInput)
				{
				}
			}

			if (bResult)
			{
				if (ReadTimecodeCounter > 0)
				{
					bResult = InChannelOptions.TimecodeFormat == ETimecodeFormat::TCF_None || InChannelOptions.TimecodeFormat == ChannelOptions.TimecodeFormat;
				}
			}

			return bResult;
		}

		bool FInputChannel::Initialize(const FInputChannelOptions& InChannelOptions)
		{
			ChannelOptions = InChannelOptions;
			if (!InitializeDeckLink())
			{
				return false;
			}

			if (!InitializeInterfaces())
			{
				return false;
			}

			// Determine whether the DeckLink device supports input format detection
			BOOL bSupportsInputFormatDetection = false;
			HRESULT Result = DeckLinkAttributes->GetFlag(BMDDeckLinkSupportsInputFormatDetection, &bSupportsInputFormatDetection);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Couldn't get the attribute flags for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			if (bSupportsInputFormatDetection)
			{
				VideoInputFlags = bmdVideoInputEnableFormatDetection;
			}

			if (!VerifyDeviceUsage())
			{
				return false;
			}

			// Always start with 8bits YUV, maybe in the future we will have to support RGB (the SDI stream is an RGB444 and not a YCbCr422)
			PixelFormat = ENUM(BMDPixelFormat)::bmdFormat8BitYUV;
			DisplayMode = static_cast<BMDDisplayMode>(InChannelOptions.FormatInfo.DisplayMode);

			if (!VerifyFormatSupport())
			{
				return false;
			}

			// Create an instance of notification callback
			NotificationCallback = new FInputChannelNotificationCallback(this, DeckLinkInput, DeckLinkStatus);
			if (NotificationCallback == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not create notification callback object for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			// Set the callback object to the DeckLink device's input interface
			Result = DeckLinkInput->SetCallback(NotificationCallback);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not set callback for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			return true;
		}

		void FInputChannel::Reinitialize()
		{
			DisplayMode = static_cast<BMDDisplayMode>(ChannelOptions.FormatInfo.DisplayMode);
			// Pause streams then follow the same pattern as in Initialize()
			DeckLinkInput->PauseStreams();
			DeckLinkInput->EnableVideoInput(DisplayMode, PixelFormat, VideoInputFlags);
			DeckLinkInput->FlushStreams();
			DeckLinkInput->StartStreams();
		}

		bool FInputChannel::InitializeDeckLink()
		{
			DeckLink = Helpers::GetDeviceForIndex(ChannelInfo.DeviceIndex);
			if (DeckLink == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not find the device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			return true;
		}

		bool FInputChannel::InitializeInterfaces()
		{
			check(DeckLink);

			// Obtain the Attributes interface for the DeckLink device
			HRESULT Result = DeckLink->QueryInterface(IID_IDeckLinkProfileAttributes, (void**)&DeckLinkAttributes);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkAttributes interface for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			// Obtain the input interface for the DeckLink device
			Result = DeckLink->QueryInterface(IID_IDeckLinkInput, (void**)&DeckLinkInput);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkInput interface for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			// Obtain the configuration interface for the DeckLink device
			Result = DeckLink->QueryInterface(IID_IDeckLinkConfiguration, (void**)&DeckLinkConfiguration);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkConfiguration interface for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			// Obtain the status interface for the DeckLink device
			Result = DeckLink->QueryInterface(IID_IDeckLinkStatus, (void**)&DeckLinkStatus);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkStatus interface for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			return true;
		}

		bool FInputChannel::VerifyFormatSupport()
		{
			BOOL bSupported = FALSE;
			HRESULT Result = DeckLinkInput->DoesSupportVideoMode(bmdVideoConnectionUnspecified, DisplayMode, PixelFormat, bmdNoVideoInputConversion, bmdSupportedVideoModeDefault, nullptr, &bSupported);
			if (Result != S_OK) 
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not check if the video mode is supported on device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			ReferencePtr<IDeckLinkDisplayMode> DeckLinkDisplayMode;
			Result = DeckLinkInput->GetDisplayMode(DisplayMode, (IDeckLinkDisplayMode**)&DeckLinkDisplayMode);
			if (Result != S_OK || DeckLinkDisplayMode == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not get the display mode on device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			if (!bSupported)
			{
				UE_LOGF(LogBlackmagicCore, Error,"The desired video mode '%s' is not supported on device '%d'.", BlackmagicPlatform::GetName(DeckLinkDisplayMode.Get()).c_str(), ChannelInfo.DeviceIndex);
				return false;
			}

			//Grab useful settings from actual display mode that is going to be used
			BMDTimeValue Denominator;
			BMDTimeScale Numerator;
			Result = DeckLinkDisplayMode->GetFrameRate(&Denominator, &Numerator);
			if (Result != S_OK || Denominator == 0)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not get a valid frame rate for input on device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			ChannelOptions.FormatInfo.FrameRateNumerator = (uint32_t)Numerator;
			ChannelOptions.FormatInfo.FrameRateDenominator = (uint32_t)Denominator;

			const BMDFieldDominance FieldDominance = DeckLinkDisplayMode->GetFieldDominance();
			Helpers::BMDFieldDominanceToEFieldDominance(DeckLinkDisplayMode->GetFieldDominance(), ChannelOptions.FormatInfo.FieldDominance);

			ChannelOptions.FormatInfo.Width = DeckLinkDisplayMode->GetWidth();
			ChannelOptions.FormatInfo.Height = DeckLinkDisplayMode->GetHeight();

			return true;
		}

		bool FInputChannel::VerifyDeviceUsage()
		{
			int64_t BusyStatus = 0;
			HRESULT Result = DeckLinkStatus->GetInt(bmdDeckLinkStatusBusy, &BusyStatus);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Couldn't get the Busy status for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			//Verify if we're already opened as input
			if ((BusyStatus & bmdDeviceCaptureBusy) != 0)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Device %d can't be used because it's already being used as an input.", ChannelInfo.DeviceIndex);
				return false;
			}

			int64_t DuplexStatus = 0;
			Result = DeckLinkAttributes->GetInt(BMDDeckLinkDuplex, &DuplexStatus);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Couldn't get the DuplexMode status for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			if (DuplexStatus == ENUM(BMDDuplexMode)::bmdDuplexInactive)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Device %d is inactive.", ChannelInfo.DeviceIndex);
				return false;
			}

			//Device we're on can only do playback or capture at the same time. We can't open it for output if it's the case.
			if (DuplexStatus == ENUM(BMDDuplexMode)::bmdDuplexHalf)
			{
				if ((BusyStatus & bmdDevicePlaybackBusy) != 0)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Device %d can't be used because it's already being used as an output and it can't do input at the same time.", ChannelInfo.DeviceIndex);
					return false;
				}
			}

			return true;
		}

		bool FInputChannel::InitializeAudio()
		{
			check(DeckLinkInput);

			HRESULT Result = DeckLinkInput->EnableAudioInput(bmdDefaultAudioSampleRate, DefaultAudioSampleType, ChannelOptions.NumberOfAudioChannel);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not enable the audio input on device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			return true;
		}

		bool FInputChannel::InitializeLTC()
		{
			check(DeckLinkAttributes);
			check(DeckLinkConfiguration);

			BOOL bSupported = false;
			HRESULT Result = DeckLinkAttributes->GetFlag(BMDDeckLinkHasLTCTimecodeInput, &bSupported);
;
			if (Result != S_OK || !bSupported)
			{
				UE_LOGF(LogBlackmagicCore, Error,"The device '%d' doesn't have a dedicated LTC input - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			Result = DeckLinkConfiguration->SetFlag(bmdDeckLinkConfigUseDedicatedLTCInput, true);
			return Result == S_OK;
		}

		void FInputChannel::ReleaseLTC()
		{
			check(DeckLinkConfiguration);

			DeckLinkConfiguration->SetFlag(bmdDeckLinkConfigUseDedicatedLTCInput, false);
		}

		bool FInputChannel::InitializeStream()
		{
			check(DeckLinkInput);

			// Enable video input with a default video mode and the automatic format detection feature enabled
			HRESULT Result = DeckLinkInput->EnableVideoInput(DisplayMode, PixelFormat, VideoInputFlags);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not enable video input for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			Result = DeckLinkInput->FlushStreams();
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not flush the streams video for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			// Start capture
			Result = DeckLinkInput->StartStreams();
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not start the streams video for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}
			bStreamStarted = true;

			return true;
		}

		FInputChannel::~FInputChannel()
		{
			if (DeckLinkInput)
			{
				// Stop capture
				if (bStreamStarted)
				{
					DeckLinkInput->StopStreams();
					DeckLinkInput->DisableAudioInput();
					DeckLinkInput->DisableVideoInput();
				}
			}

			if (NotificationCallback && DeckLinkInput)
			{
				DeckLinkInput->SetCallback(nullptr);
			}

			if (NotificationCallback)
			{
				NotificationCallback->Release();
			}

			//Wait for activity on the bus to be over.
			if (DeckLink)
			{
				if (DeckLinkStatus)
				{
					static const std::chrono::milliseconds TimeToWait = std::chrono::milliseconds(500);
					const std::chrono::high_resolution_clock::time_point TargetTime = std::chrono::high_resolution_clock::now() + TimeToWait;

					int64_t BusyStatus = bmdDeviceCaptureBusy;
					while ((BusyStatus & bmdDeviceCaptureBusy) != 0 && std::chrono::high_resolution_clock::now() < TargetTime)
					{
						DeckLinkStatus->GetInt(bmdDeckLinkStatusBusy, &BusyStatus);
					}

					if ((BusyStatus & bmdDeviceCaptureBusy) != 0)
					{
						UE_LOGF(LogBlackmagicCore, Warning,"Could not stop capture activity on device '%d' while closing it.", ChannelInfo.DeviceIndex);
					}
					DeckLinkStatus->Release();
				}
			}

			if (DeckLinkConfiguration)
			{
				DeckLinkConfiguration->Release();
			}

			if (DeckLinkAttributes)
			{
				DeckLinkAttributes->Release();
			}

			if (DeckLinkInput)
			{
				DeckLinkInput->Release();
			}

			if (DeckLink)
			{
				DeckLink->Release();
			}
		}

		size_t FInputChannel::NumberOfListeners()
		{
			std::lock_guard<std::mutex> Lock(ListenerMutex);
			return Listeners.size();
		}

		bool FInputChannel::InitializeListener(FUniqueIdentifier InIdentifier, const FInputChannelOptions& InChannelOptions, ReferencePtr<IInputEventCallback> InCallback)
		{
			FListener Listener;
			Listener.Identifier = InIdentifier;
			Listener.Callback = std::move(InCallback);
			Listener.Option = InChannelOptions;

			bool bResult = InitializeListener_Helper(Listener);
			Listener.Callback->OnInitializationCompleted(bResult);
			if (bResult)
			{
				std::lock_guard<std::mutex> Lock(ListenerMutex);
				Listeners.push_back(Listener);
				std::sort(std::begin(Listeners), std::end(Listeners), [](const FListener& lhs, const FListener& rhs) { return lhs.Option.CallbackPriority > rhs.Option.CallbackPriority; });
			}
			return bResult;
		}

		bool FInputChannel::InitializeListener_Helper(const FListener InListener)
		{
			//Pausing stream is only required when adding audio to an already opened stream
			const bool bNeedToPauseStream = bStreamStarted && (InListener.Option.bReadAudio && ReadAudioCounter == 0);

			if (bNeedToPauseStream)
			{
				HRESULT Result = DeckLinkInput->PauseStreams();
				if (Result != S_OK)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Could not pause the the streams video for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
					return false;
				}
			}

			if (InListener.Option.TimecodeFormat != ETimecodeFormat::TCF_None)
			{
				if (ReadTimecodeCounter == 0)
				{
					Helpers::ETimecodeFormatToBMDTimecodeFormat(InListener.Option.TimecodeFormat, DeviceTimecodeFormat);
					ChannelOptions.TimecodeFormat = InListener.Option.TimecodeFormat;
				}

				++ReadTimecodeCounter;
			}

			bool bResult = true;
			if (bResult && InListener.Option.bReadVideo)
			{
				if (ReadVideoCounter == 0)
				{
					if (!Helpers::EPixelFormatToBMDPixelFormat(InListener.Option.PixelFormat, PixelFormat))
					{
						UE_LOGF(LogBlackmagicCore, Error,"Could not find the pixel format '%d'.", (int32_t)InListener.Option.PixelFormat);
						bResult = false;
					}

					// Copy all the video options
					ChannelOptions.PixelFormat = InListener.Option.PixelFormat;
				}

				if (bResult)
				{
					++ReadVideoCounter;
				}
			}

			if (bResult && InListener.Option.bReadAudio)
			{
				if (ReadAudioCounter == 0)
				{
					// Copy all the audio options
					ChannelOptions.NumberOfAudioChannel = InListener.Option.NumberOfAudioChannel;

					// Initialize the audio
					if (!InitializeAudio())
					{
						bResult = false;
					}
				}

				if (bResult)
				{
					++ReadAudioCounter;
				}
			}

			if (bResult && InListener.Option.bUseTheDedicatedLTCInput)
			{
				if (ReadLTCCounter == 0)
				{
					// Copy all the LTC options

					// Initialize the LTC
					if (!InitializeLTC())
					{
						bResult = false;
					}
				}

				if (bResult)
				{
					++ReadLTCCounter;
				}
			}

			bool bInitializeStreamResult = true;
			if (bNeedToPauseStream || !bStreamStarted)
			{
				bInitializeStreamResult = InitializeStream();
			}
			return bInitializeStreamResult && bResult;
		}

		void FInputChannel::UninitializeListener(FUniqueIdentifier InIdentifier)
		{
			bool bFound = false;
			FListener FoundListener;

			{
				std::lock_guard<std::mutex> Lock(ListenerMutex);
				auto FoundItt = std::find_if(std::begin(Listeners), std::end(Listeners), [=](const FListener& InOther) { return InOther.Identifier == InIdentifier; });
				if (FoundItt != std::end(Listeners))
				{
					bFound = true;
					FoundListener = *FoundItt;
					Listeners.erase(FoundItt);
				}
			}

			if (bFound)
			{
				UninitializeListener_Helper(FoundListener);
				FoundListener.Callback->OnShutdownCompleted();
			}
		}

		bool FInputChannel::UninitializeListener_Helper(const FListener InListener)
		{
			if (InListener.Option.bUseTheDedicatedLTCInput)
			{
				--ReadLTCCounter;
				if (ReadLTCCounter == 0)
				{
					ReleaseLTC();
				}
			}

			if (InListener.Option.bReadVideo)
			{
				--ReadVideoCounter;
			}

			if (InListener.Option.bReadAudio)
			{
				--ReadAudioCounter;
				//We're not disabling audio from the input because it requires stopping the stream which breaks previously configured genlocking mechanism.
			}

			if (InListener.Option.TimecodeFormat != BlackmagicDesign::ETimecodeFormat::TCF_None)
			{
				--ReadTimecodeCounter;
				if (ReadTimecodeCounter == 0)
				{
					ChannelOptions.TimecodeFormat = ETimecodeFormat::TCF_None;
				}
			}

			return true;
		}
	}

};
