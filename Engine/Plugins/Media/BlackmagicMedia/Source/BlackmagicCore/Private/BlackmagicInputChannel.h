// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.h"
#include <chrono>

namespace BlackmagicDesign
{
	namespace Private
	{
		class FInputChannel;

		class FInputChannelNotificationCallback : public IDeckLinkInputCallback
		{
		public:
			FInputChannelNotificationCallback(FInputChannel* InOwner, IDeckLinkInput* InDeckLinkInput, IDeckLinkStatus* InDeckLinkStatus);
			virtual ~FInputChannelNotificationCallback() = default;

			HRESULT STDMETHODCALLTYPE QueryInterface(REFIID iid, LPVOID *ppv);

			ULONG STDMETHODCALLTYPE AddRef();
			ULONG STDMETHODCALLTYPE Release();

			HRESULT STDMETHODCALLTYPE VideoInputFormatChanged(BMDVideoInputFormatChangedEvents InNotificationEvents, IDeckLinkDisplayMode* InNewDisplayMode, BMDDetectedVideoInputFormatFlags InDetectedSignalFlags);
			HRESULT STDMETHODCALLTYPE VideoInputFrameArrived(IDeckLinkVideoInputFrame* InVideoFrame, IDeckLinkAudioInputPacket* InAudioPacket);

		private:
			void GetHDRMetaData(IInputEventCallback::FFrameReceivedInfo& FrameInfo, IDeckLinkVideoInputFrame* InVideoFrame);
			double GetFloatMetaData(IDeckLinkVideoFrameMetadataExtensions* MetadataExtensions, BMDDeckLinkFrameMetadataID MetaDataID);
			bool IsHDRLoggingOK();

			FInputChannel* InputChannel;
			IDeckLinkInput* DeckLinkInput;
			IDeckLinkStatus* DeckLinkStatus;
			std::atomic_char32_t RefCount;
			int64_t FrameNumber;
			bool bHasProcessedVideoInput = false;
			int32_t HDRLogCount;
			std::chrono::time_point<std::chrono::system_clock> HDRLogResetCountTime;
		};

		class FInputChannel
		{
		private:
			struct FListener
			{
				FUniqueIdentifier Identifier;
				ReferencePtr<IInputEventCallback> Callback;
				FInputChannelOptions Option;
			};

		public:
			FInputChannel(const FChannelInfo& InChannelInfo);
			~FInputChannel();
			FInputChannel(const FInputChannel&) = delete;
			FInputChannel& operator=(const FInputChannel&) = delete;

			bool Initialize(const FInputChannelOptions& InChannelOptions);
			void Reinitialize();

			const FChannelInfo& GetChannelInfo() const { return ChannelInfo; }

			bool IsCompatible(const FChannelInfo& InChannelInfo, const FInputChannelOptions& InChannelOptions) const;

			size_t NumberOfListeners();
			bool InitializeListener(FUniqueIdentifier InIdentifier, const FInputChannelOptions& InChannelOptions, ReferencePtr<IInputEventCallback> InCallback);
			void UninitializeListener(FUniqueIdentifier InIdentifier);

		private:
			bool InitializeListener_Helper(const FListener InListener);
			bool UninitializeListener_Helper(const FListener InListener);

			bool InitializeInterfaces();
			bool VerifyFormatSupport();
			bool VerifyDeviceUsage();
			bool InitializeDeckLink();
			bool InitializeAudio();
			bool InitializeLTC();
			void ReleaseLTC();
			bool InitializeStream();

		private:
			friend FInputChannelNotificationCallback;

			FChannelInfo ChannelInfo;
			FInputChannelOptions ChannelOptions;
			int32_t ReadAudioCounter;
			int32_t ReadLTCCounter;
			int32_t ReadVideoCounter;
			int32_t ReadTimecodeCounter;


			IDeckLink* DeckLink;
			IDeckLinkInput* DeckLinkInput;
			IDeckLinkProfileAttributes* DeckLinkAttributes;
			IDeckLinkConfiguration* DeckLinkConfiguration;
			FInputChannelNotificationCallback* NotificationCallback;
			IDeckLinkStatus* DeckLinkStatus;
			bool bStreamStarted;

			BMDVideoInputFlags VideoInputFlags;
			BMDPixelFormat PixelFormat;
			BMDDisplayMode DisplayMode;

			BMDTimecodeFormat DeviceTimecodeFormat;

			std::vector<FListener> Listeners;
			std::mutex ListenerMutex;

			volatile bool bIsWaitingForInterlaceEvent;

			bool bCanProcessOddFrame;
			FTimecode PreviousReceivedTimecode;
		};
	}
}
