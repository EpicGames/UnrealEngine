// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicLib.h"

#include "Common.h"

#include "BlackmagicDevice.h"
#include "BlackmagicDeviceScanner.h"
#include "BlackmagicHelper.h"
#include "BlackmagicVideoFormats.h"

#if PLATFORM_WINDOWS
#include "GPUTextureTransfer.h"
#endif

namespace BlackmagicDesign
{
	UE::GPUTextureTransfer::ITextureTransfer* TextureTransfer = nullptr;

	/* FTimecode
	*****************************************************************************/
	FUniqueIdentifier::FUniqueIdentifier()
		: Identifier(Private::UniqueIdentifierGenerator::InvalidId)
	{}

	FUniqueIdentifier::FUniqueIdentifier(int32_t InIdentifier)
		: Identifier(InIdentifier)
	{}

	bool FUniqueIdentifier::IsValid() const
	{
		return Identifier != Private::UniqueIdentifierGenerator::InvalidId;
	}
	
	bool FUniqueIdentifier::operator== (const FUniqueIdentifier& rhs) const
	{
		return Identifier == rhs.Identifier;
	}

	/* FTimecode 
	*****************************************************************************/
	FTimecode::FTimecode()
		: Hours(0)
		, Minutes(0)
		, Seconds(0)
		, Frames(0)
		, bIsDropFrame(false)
	{ }

	bool FTimecode::operator== (const FTimecode& Other) const
	{
		return Other.Hours == Hours
			&& Other.Minutes == Minutes
			&& Other.Seconds == Seconds
			&& Other.Frames == Frames
			&& Other.bIsDropFrame == bIsDropFrame;
	}

	/* FChannelInfo 
	*****************************************************************************/
	bool FChannelInfo::operator==(FChannelInfo& Other) const
	{
		return DeviceIndex == Other.DeviceIndex;
	}

	/* FHDRMetaData
	*****************************************************************************/
	FHDRMetaData::FHDRMetaData()
		: bIsAvailable(false)
		, ColorSpace(EHDRMetaDataColorspace::Rec601)
		, EOTF(EHDRMetaDataEOTF::SDR)
		, WhitePointX(0.0)
		, WhitePointY(0.0)
		, DisplayPrimariesRedX(0.0)
		, DisplayPrimariesRedY(0.0)
		, DisplayPrimariesGreenX(0.0)
		, DisplayPrimariesGreenY(0.0)
		, DisplayPrimariesBlueX(0.0)
		, DisplayPrimariesBlueY(0.0)
		, MaxDisplayLuminance(0)
		, MinDisplayLuminance(0)
        , MaxContentLightLevel(0)
        , MaxFrameAverageLightLevel(0)
	
	{ }

	/* FInputChannelOptions
	*****************************************************************************/
	FInputChannelOptions::FInputChannelOptions()
		: CallbackPriority(0)
		, bReadVideo(false)
		, PixelFormat(EPixelFormat::pf_8Bits)
		, bReadAudio(false)
		, NumberOfAudioChannel(2)
		, bUseTheDedicatedLTCInput(false)
		, bAutoDetect(false)
	{
	}

	/* FOutputChannelOptions 
	*****************************************************************************/
	FOutputChannelOptions::FOutputChannelOptions()
		: CallbackPriority(0)
		, PixelFormat(EPixelFormat::pf_8Bits)
		, AudioSampleRate(EAudioSampleRate::SR_48kHz)
		, AudioBitDepth(EAudioBitDepth::Signed_32Bits)
		, NumAudioChannels(EAudioChannelConfiguration::Channels_2)
		, TimecodeFormat(ETimecodeFormat::TCF_None)
		, bOutputKey(false)
		, bOutputVideo(true)
		, bOutputAudio(false)
		, bOutputInterlacedFieldsTimecodeNeedToMatch(false)
		, bOutputInterlaceAsProgressive(false)
		, bOutputInterlaceOnEvenFrames(false)
		, bLogDropFrames(false)
		, bUseGPUDMA(false)
		, bScheduleInDifferentThread(false)
		, bEncodeTimecodeInTexel(false)
	{
	}

	/* IInputEventCallback 
	*****************************************************************************/
	IInputEventCallback::~IInputEventCallback()
	{ }

	IInputEventCallback::FFrameReceivedInfo::FFrameReceivedInfo()
		: bHasInputSource(false)
		, FrameNumber(0)
		, bHaveTimecode(false)
		, VideoBuffer(nullptr)
		, AudioBuffer(nullptr)
	{ }

	/* IOutputEventCallback 
	*****************************************************************************/
	IOutputEventCallback::~IOutputEventCallback()
	{ }

	IOutputEventCallback::FFrameSentInfo::FFrameSentInfo()
		: FramesLost(0)
	{ }


	/* BlackmagicDeviceScanner 
	*****************************************************************************/

	BlackmagicDeviceScanner::BlackmagicDeviceScanner()
	{
		Scanner = new BlackmagicDesign::Private::DeviceScanner();
	}

	BlackmagicDeviceScanner::~BlackmagicDeviceScanner()
	{
		delete Scanner;
	}

	int32_t BlackmagicDeviceScanner::GetNumDevices() const
	{
		return Scanner->GetNumDevices();
	}

	bool BlackmagicDeviceScanner::GetDeviceTextId(int32_t InDeviceIndex, FormatedTextType& OutTextId) const
	{
		return Scanner->GetDeviceTextId(InDeviceIndex, OutTextId);
	}

	bool BlackmagicDeviceScanner::GetDeviceInfo(int32_t InDeviceIndex, DeviceInfo& OutDeviceInfo) const
	{
		return Scanner->GetDeviceInfo(InDeviceIndex, OutDeviceInfo);
	}

	/* BlackmagicVideoFormats implementation
	*****************************************************************************/
	BlackmagicVideoFormats::VideoFormatDescriptor::VideoFormatDescriptor()
		: bIsValid(false)
	{}

	BlackmagicVideoFormats::BlackmagicVideoFormats(int32_t InDeviceId, bool bForOutput)
	{
		Formats = new BlackmagicDesign::Private::VideoFormatsScanner(InDeviceId, bForOutput);
	}

	BlackmagicVideoFormats::~BlackmagicVideoFormats()
	{
		delete Formats;
	}

	int32_t BlackmagicVideoFormats::GetNumSupportedFormat() const
	{
		return (int32_t)Formats->FormatList.size();
	}

	BlackmagicVideoFormats::VideoFormatDescriptor BlackmagicVideoFormats::GetSupportedFormat(int32_t InIndex) const
	{
		if (InIndex < Formats->FormatList.size())
		{
			return Formats->FormatList[InIndex];
		}
		return BlackmagicVideoFormats::VideoFormatDescriptor();
	}

	/* Initialization methods
	*****************************************************************************/

	bool ApiInitialization()
	{
		bool bResult = BlackmagicPlatform::InitializeAPI();
		if (bResult)
		{
			Private::FDevice::CreateInstance();
			bResult = Private::FDevice::GetDevice() != nullptr;
		}
		return bResult;
	}

	void ApiUninitialization()
	{
		if (Private::FDevice::GetDevice() != nullptr)
		{
			Private::FDevice::DestroyInstance();
		}
		BlackmagicPlatform::ReleaseAPI();
	}

	FUniqueIdentifier RegisterCallbackForChannel(const FChannelInfo& InChannelInfo, const FInputChannelOptions& InChannelOptions, ReferencePtr<IInputEventCallback> InCallback)
	{
		if (Private::FDevice::GetDevice() == nullptr)
		{
			UE_LOGF(LogBlackmagicCore, Error,"The api is not initialized");
			return FUniqueIdentifier();
		}

		return Private::FDevice::GetDevice()->RegisterChannel(InChannelInfo, InChannelOptions, std::move(InCallback));
	}

	void UnregisterCallbackForChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier)
	{
		if (Private::FDevice::GetDevice() == nullptr)
		{
			UE_LOGF(LogBlackmagicCore, Error,"The api is not initialized");
			return;
		}

		Private::FDevice::GetDevice()->UnregisterCallbackForChannel(InChannelInfo, InIdentifier);
	}

	FUniqueIdentifier RegisterOutputChannel(const FChannelInfo& InChannelInfo, const FOutputChannelOptions& InChannelOptions, ReferencePtr<IOutputEventCallback> InCallback)
	{
		if (Private::FDevice::GetDevice() == nullptr)
		{
			UE_LOGF(LogBlackmagicCore, Error,"The api is not initialized");
			return FUniqueIdentifier();
		}

		return Private::FDevice::GetDevice()->RegisterOutputChannel(InChannelInfo, InChannelOptions, std::move(InCallback));
	}

	void UnregisterOutputChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier, bool bCallCompleted)
	{
		if (Private::FDevice::GetDevice() == nullptr)
		{
			UE_LOGF(LogBlackmagicCore, Error,"The api is not initialized");
			return;
		}

		Private::FDevice::GetDevice()->UnregisterOutputChannel(InChannelInfo, InIdentifier);
	}

	bool SendVideoFrameData(const FChannelInfo& InChannelInfo, const FFrameDescriptor& InFrame)
	{
		if (Private::FDevice::GetDevice() == nullptr)
		{
			UE_LOGF(LogBlackmagicCore, Error,"The api is not initialized");
			return false;
		}

		return Private::FDevice::GetDevice()->SendVideoFrameData(InChannelInfo, InFrame);
	}

	bool SendVideoFrameData(const FChannelInfo& InChannelInfo, FFrameDescriptor_GPUDMA& InFrame)
	{
#if PLATFORM_WINDOWS
		if (Private::FDevice::GetDevice() == nullptr)
		{
			UE_LOGF(LogBlackmagicCore, Error,"The api is not initialized");
			return false;
		}

		return Private::FDevice::GetDevice()->SendVideoFrameData(InChannelInfo, InFrame);
#else
		UE_LOGF(LogBlackmagicCore, Error,"GPU DMA is not available on linux.");
		return false;
#endif
	}

	bool ShouldCaptureThisFrame(const FChannelInfo& InChannelInfo, ::FTimecode Timecode, uint32 FrameNumber)
	{
		return Private::FDevice::GetDevice()->ShouldCaptureThisFrame(InChannelInfo, Timecode, FrameNumber);
	}

	bool SendAudioSamples(const FChannelInfo& InChannelInfo, const FAudioSamplesDescriptor& InSamples)
	{
		if (Private::FDevice::GetDevice() == nullptr)
		{
			UE_LOGF(LogBlackmagicCore, Error,"The api is not initialized");
			return false;
		}

		return Private::FDevice::GetDevice()->SendAudioSamples(InChannelInfo, InSamples);
	}
};
