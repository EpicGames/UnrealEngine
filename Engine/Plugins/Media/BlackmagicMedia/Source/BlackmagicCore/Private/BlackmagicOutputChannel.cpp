// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicOutputChannel.h"

#include "Common.h"

#include "BlackmagicHelper.h"
#include "MediaIOCoreEncodeTime.h"
#include "SSE2MemCpy.h"

#include <chrono>
#include <iostream>
#include <fstream>


#if PLATFORM_WINDOWS
#include "GPUTextureTransfer.h"
#endif

#define AUDIO_DEBUGGING 0

namespace BlackmagicDesign
{
	namespace Private
	{
		// Do not use all the available thread. Keep 4 for the 4 Unreal Engine's virtual thread.
		const int32_t NumberOfThread = std::thread::hardware_concurrency() - 4;

		FOutputChannelNotificationCallback::FOutputChannelNotificationCallback(FOutputChannel* InOwner, IDeckLinkOutput* InDeckLinkOutput)
			: OutputChannel(InOwner)
			, DeckLinkOutput(InDeckLinkOutput)
			, RefCount(1)
		{
		}

		HRESULT FOutputChannelNotificationCallback::QueryInterface(REFIID iid, LPVOID *ppv)
		{
			return E_NOINTERFACE;
		}

		ULONG FOutputChannelNotificationCallback::AddRef()
		{
			return ++RefCount;
		}

		ULONG FOutputChannelNotificationCallback::Release()
		{
			--RefCount;
			if (RefCount == 0)
			{
				delete this;
				return 0;
			}
			return RefCount;
		}

		HRESULT FOutputChannelNotificationCallback::ScheduledFrameCompleted(IDeckLinkVideoFrame* InCompletedFrame, BMDOutputFrameCompletionResult InResult)
		{
			check(OutputChannel);
			OutputChannel->OnScheduledFrameCompleted(InCompletedFrame, InResult);

			return S_OK;
		}

		HRESULT FOutputChannelNotificationCallback::ScheduledPlaybackHasStopped()
		{
			check(OutputChannel);

			std::lock_guard<std::mutex> Lock(OutputChannel->CallbackMutex);
			for (FOutputChannel::FListener& Listener : OutputChannel->Listeners)
			{
				Listener.Callback->OnPlaybackStopped();
			}

			return S_OK;
		}

		FOutputChannelRenderAudioCallback::FOutputChannelRenderAudioCallback(FOutputChannel* InOwner, IDeckLinkOutput* InDeckLinkOutput)
			: OutputChannel(InOwner)
			, DeckLinkOutput(InDeckLinkOutput)
			, RefCount(1)
		{
		}

		HRESULT FOutputChannelRenderAudioCallback::RenderAudioSamples(BOOL Preroll)
		{
			check(OutputChannel);
			return OutputChannel->OnRenderAudio(Preroll) ? S_OK : S_FALSE;
		}

		HRESULT FOutputChannelRenderAudioCallback::QueryInterface(REFIID iid, LPVOID* ppv)
		{
			return E_NOINTERFACE;
		}

		ULONG FOutputChannelRenderAudioCallback::AddRef()
		{
			return ++RefCount;
		}

		ULONG FOutputChannelRenderAudioCallback::Release()
		{
			--RefCount;
			if (RefCount == 0)
			{
				delete this;
				return 0;
			}
			return RefCount;
		}

		void EncodeTimecodeInTexel(BMDPixelFormat PixelFormat, BlackmagicDesign::FTimecode Timecode, void* Buffer, int32 Width, int32 Height, int32 XOffset = 0, int32 YOffset = 0)
		{
			switch (PixelFormat)
			{
			case bmdFormat8BitYUV:
			{
				FMediaIOCoreEncodeTime EncodeTime(EMediaIOCoreEncodePixelFormat::CharUYVY, Buffer, Width * 4, Width * 2, Height, XOffset, YOffset);
				EncodeTime.Render(Timecode.Hours, Timecode.Minutes, Timecode.Seconds, Timecode.Frames);
				break;
			}
			case bmdFormat8BitBGRA:
			{
				FMediaIOCoreEncodeTime EncodeTime(EMediaIOCoreEncodePixelFormat::CharBGRA, Buffer, Width * 4, Width, Height, XOffset, YOffset);
				EncodeTime.Render(Timecode.Hours, Timecode.Minutes, Timecode.Seconds, Timecode.Frames);
				break;
			}
			case bmdFormat10BitYUV:
			{
				FMediaIOCoreEncodeTime EncodeTime(EMediaIOCoreEncodePixelFormat::YUVv210, Buffer, Width * 16, Width * 6, Height, XOffset, YOffset);
				EncodeTime.Render(Timecode.Hours, Timecode.Minutes, Timecode.Seconds, Timecode.Frames);
				break;
			}
			default:
				break;
			}
		}

		/* Frame implementation
		*****************************************************************************/
		const uint32_t FOutputChannel::FOutputFrame::InvalidFrameIdentifier = static_cast<uint32_t>(-1);

		FOutputChannel::FOutputFrame::FOutputFrame()
			: InFlightCount(0)
			, bVideoLineFilled(false)
			, bVideoF2LineFilled(false) // used in interlaced
			, bVideoScheduled(false)
		{
			Clear();
		}

		FOutputChannel::FOutputFrame::~FOutputFrame()
		{
		}

		void FOutputChannel::FOutputFrame::Clear()
		{
			CopiedVideoBufferSize = 0;
			FrameIdentifier = InvalidFrameIdentifier;
			FrameIdentifierF2 = InvalidFrameIdentifier;
			bVideoLineFilled = false;
			bVideoF2LineFilled = false;
			bVideoScheduled = false;
		}



		FOutputChannel::FOutputChannel(const FChannelInfo& InChannelInfo, const FOutputChannelOptions InChannelOptions)
			: ChannelInfo(InChannelInfo)
			, ChannelOptions(InChannelOptions)
			, DeckLink(nullptr)
			, DeckLinkAttributes(nullptr)
			, DeckLinkOutput(nullptr)
			, DeckLinkConfiguration(nullptr)
			, Keyer(nullptr)
			, NotificationCallback(nullptr)
			, RenderAudioCallback(nullptr)
			, bAudioOutputStarted(false)
			, bVideoOutputStarted(false)

			, VideoOutputFlags(bmdVideoOutputFlagDefault)
			, PixelFormat(bmdFormat8BitYUV)
			, DisplayMode(bmdModeHD1080p24)

			, bOutputTimecode(false)
			, DeviceTimecodeFormat(ENUM(BMDTimecodeFormat)::bmdTimecodeRP188Any)
			, bWaitForInterlacedOddFieldActive(false)
			, bScheduleAudioSamplesActive(false)
			, bScheduleVideoActive(false)
			, InFlightFrames(0)
			, LostFrameCounter(0)
			, DroppedFrameCounter(0)
			, QueuedDroppedFrameCount(0)
			, NumBufferedAudioSamples(0)

			, bIsWaitingForInterlaceEvent(false)
			, AudioPacketPool(200)
			, bIsSSE2Available(IsSSE2Available())
			, bIsFieldAEven(true)
			, bIsFirstFetch(true)
			, bIsPrerollingAudio(true)
			, bPlaybackStopped(false)
		{

		}

		bool FOutputChannel::Initialize()
		{
			if (!InitializeDeckLink())
			{
				return false;
			}

			if (!InitializeInterfaces())
			{
				return false;
			}

			// Determine whether the DeckLink device supports playback mode
			int64_t VideoCapabilities = 0;
			HRESULT Result = DeckLinkAttributes->GetInt(ENUM(BMDDeckLinkAttributeID)::BMDDeckLinkVideoIOSupport, &VideoCapabilities);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Couldn't get the attribute flags for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			if ((VideoCapabilities & bmdDeviceSupportsPlayback) == 0)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Device '%d' doesn't support playback mode.", ChannelInfo.DeviceIndex);
				return false;
			}

			VideoOutputFlags = ENUM(BMDVideoOutputFlags)::bmdVideoOutputFlagDefault;
			if (ChannelOptions.TimecodeFormat != ETimecodeFormat::TCF_None)
			{
				VideoOutputFlags = ENUM(BMDVideoOutputFlags)::bmdVideoOutputRP188;
				if (Helpers::ETimecodeFormatToBMDTimecodeFormat(ChannelOptions.TimecodeFormat, DeviceTimecodeFormat))
				{
					bOutputTimecode = true;
				}
			}

			if (!InitializeLinkConfiguration())
			{
				return false;
			}

			if (!VerifyIfDeviceUsed())
			{
				return false;
			}

			Allocator = new CustomAllocator();
			DeckLinkOutput->SetVideoOutputFrameMemoryAllocator(Allocator);

			//Once the output interface has been fetched, try to allocate required buffers
			if (!InitializeFrameBuffer())
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not create output buffers for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			DisplayMode = static_cast<BMDDisplayMode>(ChannelOptions.FormatInfo.DisplayMode);
			if (!Helpers::EPixelFormatToBMDPixelFormat(ChannelOptions.PixelFormat, PixelFormat))
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not find the pixel format '%d'.", (int32_t)ChannelOptions.PixelFormat);
				return false;
			}

			if (!VerifyFormatSupport())
			{
				return false;
			}

			if (!InitializeKeyer())
			{
				return false;
			}

			// Create an instance of notification callback
			NotificationCallback = new FOutputChannelNotificationCallback(this, DeckLinkOutput);
			if (NotificationCallback == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not create notification callback object for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			RenderAudioCallback = new FOutputChannelRenderAudioCallback(this, DeckLinkOutput);

			// Set the callback object to the DeckLink device's Output interface
			Result = DeckLinkOutput->SetScheduledFrameCompletionCallback(NotificationCallback);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not set callback for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			if (FGPUTextureTransferModule::Get().IsEnabled() && FGPUTextureTransferModule::Get().IsInitialized())
			{
				TextureTransfer = FGPUTextureTransferModule::Get().GetTextureTransfer();
			}

			if (ChannelOptions.bOutputAudio)
			{
				if (!InitializeAudio())
				{
					return false;
				}
			}

			if (!InitializeVideo())
			{
				return false;
			}

			if (!InitializeWaitForInterlacedOddFieldEvent())
			{
				return false;
			}

			return true;
		}

		bool FOutputChannel::InitializeDeckLink()
		{
			DeckLink = Helpers::GetDeviceForIndex(ChannelInfo.DeviceIndex);
			if (DeckLink == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not find the device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			return true;
		}

		bool FOutputChannel::InitializeInterfaces()
		{
			check(DeckLink);

			// Obtain the Attributes interface for the DeckLink device
			HRESULT Result = DeckLink->QueryInterface(IID_IDeckLinkProfileAttributes, (void**)&DeckLinkAttributes);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkAttributes interface for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			// Obtain the input interface for the DeckLink device
			Result = DeckLink->QueryInterface(IID_IDeckLinkOutput, (void**)&DeckLinkOutput);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkOutput interface for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			// Obtain the configuration interface for the DeckLink device
			Result = DeckLink->QueryInterface(IID_IDeckLinkConfiguration, (void**)&DeckLinkConfiguration);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkConfiguration interface for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			return true;
		}

		bool FOutputChannel::InitializeLinkConfiguration()
		{
			check(DeckLink);
			check(DeckLinkAttributes);
			check(DeckLinkConfiguration);

			BMDLinkConfiguration Configuration;
			switch (ChannelOptions.LinkConfiguration)
			{
			case ELinkConfiguration::SingleLink:
				Configuration = bmdLinkConfigurationSingleLink;
				break;
			case ELinkConfiguration::DualLink:
				Configuration = bmdLinkConfigurationDualLink;
				break;
			case ELinkConfiguration::QuadLinkTSI:
			case ELinkConfiguration::QuadLinkSqr:
				Configuration = bmdLinkConfigurationQuadLink;
				break;
			default:
				check(false);
				return false;
			}

			HRESULT Result = DeckLinkConfiguration->SetInt(bmdDeckLinkConfigSDIOutputLinkConfiguration, (int64_t)Configuration);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not change device '%d' link mode.", ChannelInfo.DeviceIndex);
				return false;
			}

			if (Configuration == bmdLinkConfigurationQuadLink)
			{
				BOOL bCurrentValue = ChannelOptions.LinkConfiguration == ELinkConfiguration::QuadLinkSqr;
				Result = DeckLinkConfiguration->SetFlag(bmdDeckLinkConfigQuadLinkSDIVideoOutputSquareDivisionSplit, bCurrentValue);

				// Could be not supported
				if (Result == E_NOTIMPL && ChannelOptions.LinkConfiguration == ELinkConfiguration::QuadLinkSqr)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Could not change device '%d' Quad-link output. It is not supported by the device.", ChannelInfo.DeviceIndex);
					return false;
				}
				else if (Result == E_FAIL && Result == E_INVALIDARG)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Could not change device %d Quad-link output to Square Division Quad Split mode.", ChannelInfo.DeviceIndex);
					return false;
				}
			}

			return true;
		}

		bool FOutputChannel::InitializeFrameBuffer()
		{
			check(DeckLinkOutput);

			LastScheduledFrame.store(nullptr);

			bool bSuccess = true;
			if (ChannelOptions.NumberOfBuffers > 0)
			{
				const int32_t VideoWidth = ChannelOptions.FormatInfo.Width;
				const int32_t VideoHeight = ChannelOptions.FormatInfo.Height;
				int32_t VideoStride = VideoWidth * 4;
				BMDPixelFormat InputPixelFormat = bmdFormat8BitYUV;

				if (Helpers::Is8Bits(ChannelOptions.PixelFormat))
				{
					if (ChannelOptions.bOutputKey)
					{
						VideoStride = VideoWidth * 4;
						InputPixelFormat = bmdFormat8BitBGRA;
					}
					else
					{
						VideoStride = VideoWidth * 2;
						InputPixelFormat = bmdFormat8BitYUV;
					}
				}
				else
				{	
					// Padding aligned on 48 (16 and 6 at the same time)
					VideoStride = (((VideoWidth + 47) / 48) * 48) * 16 / 6;
					InputPixelFormat = bmdFormat10BitYUV;
				}

				bool bFailed = false;
				AllFrames.reserve(ChannelOptions.NumberOfBuffers);
				for (uint32_t Index = 0; Index < ChannelOptions.NumberOfBuffers; ++Index)
				{
					IDeckLinkMutableVideoFrame* NewBlackmagicFrame = nullptr;
					HRESULT Result = DeckLinkOutput->CreateVideoFrame(VideoWidth, VideoHeight, VideoStride, InputPixelFormat, bmdFrameFlagDefault, &NewBlackmagicFrame);

					if (Result == S_OK)
					{
						FOutputFrame* NewFrame = new FOutputFrame();
						NewFrame->BlackmagicFrame = ReferencePtr<FDecklinkVideoFrame>(new FDecklinkVideoFrame(NewBlackmagicFrame, ChannelOptions.HDRMetadata));
						AllFrames.push_back(NewFrame);
						FrameReadyToWrite.push_back(NewFrame);
					}
					else
					{
						bFailed = true;
						break;
					}
				}

				if (bFailed)
				{
					for (auto Frame : AllFrames)
					{
						delete Frame;
					}
					AllFrames.clear();
					bSuccess = false;
				}
			}

			return bSuccess;
		}

		void FOutputChannel::OnScheduledFrameCompleted(IDeckLinkVideoFrame* InCompletedFrame, BMDOutputFrameCompletionResult InResult)
		{
			//Notify the engine that a frame has completed to synchronize
			{
				IOutputEventCallback::FFrameSentInfo Info;
				Info.FramesLost = LostFrameCounter;
				Info.FramesDropped = DroppedFrameCounter;

				std::lock_guard<std::mutex> Lock(CallbackMutex);
				for (FOutputChannel::FListener& Listener : Listeners)
				{
					Listener.Callback->OnOutputFrameCopied(Info);
				}
			}

			//Stall for half a frame if in interlaced mode to trigger the artificial field event
			if (WaitForInterlacedOddFieldCondition)
			{
				WaitForInterlacedOddFieldCondition->notify_one();
			}

			{
				--InFlightFrames;

				FOutputFrame* FoundFrame = nullptr;
				for (FOutputFrame* FrameItt : AllFrames)
				{
					if (FrameItt->BlackmagicFrame == InCompletedFrame)
					{
						FoundFrame = FrameItt;
						break;
					}
				}

				UE_LOGF(LogBlackmagicCore, Verbose, "\tOnScheduledFrame %d, %d:%d", FoundFrame->FrameIdentifier, FoundFrame->Timecode.Seconds, FoundFrame->Timecode.Frames);

				check(FoundFrame);
				if (FoundFrame->InFlightCount == 0)
				{
					UE_LOGF(LogBlackmagicCore, Error,"No frame is currently in flight, verify that the engine framerate is high enough to support the output framerate.");
				}
				else
				{
					--FoundFrame->InFlightCount;
				}


				//If a frame was not completed correctly according to the internal frame engine, bump the internal time to keep get it back on its feet.
				//If a frame was flushed (removed from the queue) or dropped (not displayed) bump the drop frame counter
				//If a frame was displayed late, we still bump the time to avoid getting these status but it doesn't mean that a frame was not displayed.
				if (InResult != ENUM(BMDOutputFrameCompletionResult)::bmdOutputFrameCompleted)
				{
					AdvanceTime();

					if (InResult == ENUM(BMDOutputFrameCompletionResult)::bmdOutputFrameFlushed || InResult == ENUM(BMDOutputFrameCompletionResult)::bmdOutputFrameDropped)
					{
						++QueuedDroppedFrameCount;
					}
				}

				{
					// Todo: When upgrading to C++17 or moving to the engine, use a scoped lock instead
					std::lock(FrameLock, ScheduleLock);
					std::lock_guard<std::mutex> FrameLockGuard(FrameLock, std::adopt_lock);
					std::lock_guard<std::mutex> ScheduleLockGuard(ScheduleLock, std::adopt_lock);
					// Make sure we're not scheduling while checking for in flight frames, to avoid a case
					// where we re-send a frame after a new frame was scheduled.

					if (FoundFrame->InFlightCount == 0 && (InFlightFrames > 0 || FoundFrame != LastScheduledFrame))
					{
						FoundFrame->Clear();
						FrameReadyToWrite.push_back(FoundFrame);
					}
				}

				//If no frame found to be sent, resend the last one
				if (InFlightFrames <= 0)
				{
					++QueuedDroppedFrameCount;
					if (ChannelOptions.bScheduleInDifferentThread)
					{ 
						WaitForVideoScheduleCondition.notify_one();
					}

					UE_LOGF(LogBlackmagicCore, Verbose, "\tResending last frame: %d", LastScheduledFrame.load()->FrameIdentifier);
					ScheduleOutputFrame(LastScheduledFrame);
				}
				else
				{
					DroppedFrameCounter += QueuedDroppedFrameCount;
					QueuedDroppedFrameCount = 0;
				}
			}
		}

		bool FOutputChannel::OnRenderAudio(BOOL bPreRoll)
		{
			{
				uint32_t BufferedSamples = 0;
				if (DeckLinkOutput->GetBufferedAudioSampleFrameCount(&BufferedSamples) != S_OK)
				{
					return false;
				}


#if AUDIO_DEBUGGING
				auto millisec_since_epoch = std::chrono::duration_cast<std::chrono::milliseconds>(std::chrono::system_clock::now().time_since_epoch()).count();
				UE_LOGF(LogBlackmagicCore, Warning,"RenderAudio, %d, NumSamples: %d", millisec_since_epoch, BufferedSamples);
#endif

				std::unique_lock<std::mutex> lock(WaitForAudioRenderMutex);
				NumBufferedAudioSamples = BufferedSamples;
			}

			WaitForAudioRenderCondition.notify_one();
			return true;
		}

		void FOutputChannel::UnregisterBuffers()
		{
#if PLATFORM_WINDOWS
			if (bRegisteredDMABuffers && TextureTransfer && ChannelOptions.bUseGPUDMA)
			{
				for (FOutputFrame* Frame : AllFrames)
				{
					if (Frame)
					{
						void* Buffer = nullptr;
						Frame->BlackmagicFrame->GetBytes(&Buffer);
						TextureTransfer->UnregisterBuffer(Buffer);
					}
				}
			}
#endif
		}

		bool FOutputChannel::ScheduleOutputFrame(FOutputFrame* InFrame)
		{
			std::lock_guard<std::mutex> Lock(ScheduleLock);
			AdvanceTime();

			if (bOutputTimecode)
			{
				EmbedTimecodeInFrame(InFrame);
			}

			void* DestinationBuffer;
			InFrame->BlackmagicFrame->GetBytes(&DestinationBuffer);

			const int32_t VideoHeight = ChannelOptions.FormatInfo.Height;
			const uint32_t Stride = InFrame->BlackmagicFrame->GetRowBytes();
			const uint32_t VideoWidth = ChannelOptions.PixelFormat == EPixelFormat::pf_8Bits ? Stride / 4 : Stride / 16;
			
			if (ChannelOptions.FormatInfo.FieldDominance == EFieldDominance::Progressive)
			{
				if (ChannelOptions.bEncodeTimecodeInTexel)
				{
					EncodeTimecodeInTexel(PixelFormat, InFrame->Timecode, DestinationBuffer, VideoWidth, VideoHeight);
				}
			}
			else
			{
				if (ChannelOptions.bEncodeTimecodeInTexel)
				{
					EncodeTimecodeInTexel(PixelFormat, InFrame->Timecode, DestinationBuffer, VideoWidth, VideoHeight);
					EncodeTimecodeInTexel(PixelFormat, InFrame->Timecode2, DestinationBuffer, VideoWidth, VideoHeight, 0, 20);
				}

				if (InFrame->Timecode.Frames % 2 == 0)
				{
					UE_LOGF(LogBlackmagicCore, VeryVerbose, "Scheduled frame starting with even timecode. (%d)", InFrame->Timecode.Frames);
				}
				else
				{
					UE_LOGF(LogBlackmagicCore, VeryVerbose, "Scheduled frame starting with odd timecode. (%d)", InFrame->Timecode.Frames);
				}
			}

			const BMDTimeValue Duration = ChannelOptions.FormatInfo.FrameRateDenominator;
			const BMDTimeScale Scale = ChannelOptions.FormatInfo.FrameRateNumerator;
			HRESULT ScheduleVideoResult = DeckLinkOutput->ScheduleVideoFrame(static_cast<IDeckLinkVideoFrame*>(InFrame->BlackmagicFrame.Get()), OutputTime, Duration, Scale);

#if PLATFORM_WINDOWS
			// Don't call end sync complete if re-sending frame.
			if (InFrame != LastScheduledFrame)
			{
				void* Buffer = nullptr;
				InFrame->BlackmagicFrame->GetBytes(&Buffer);
				if (ChannelOptions.bUseGPUDMA && TextureTransfer)
				{
					TextureTransfer->EndSync(Buffer);
				}
			}
#endif

			if (ScheduleVideoResult == S_OK)
			{
				++InFlightFrames;
				++InFrame->InFlightCount;
				LastScheduledFrame.store(InFrame);
				return true;
			}

			UE_LOGF(LogBlackmagicCore, Error,"Could not schedule frame for Output device %d", ChannelInfo.DeviceIndex);
			return false;
		}

		void FOutputChannel::AdvanceTime()
		{
			OutputTime += ChannelOptions.FormatInfo.FrameRateDenominator;
		}

		uint32_t FOutputChannel::GetNumAudioSamplesPerFrame()
		{
			const uint32_t SampleRate = ChannelOptions.AudioSampleRate == EAudioSampleRate::SR_48kHz ? 48000 : 96000;
			return SampleRate * ChannelOptions.FormatInfo.FrameRateDenominator / ChannelOptions.FormatInfo.FrameRateNumerator;
		}

		uint8_t FOutputChannel::GetAudioBitDepth()
		{
			return ChannelOptions.AudioBitDepth == EAudioBitDepth::Signed_16Bits ? 16 : 32;
		}

		void FOutputChannel::VideoThread_ScheduleVideo()
		{
			if (TextureTransfer)
			{
				TextureTransfer->ThreadPrep();
			}
			while (true)
			{
				if (!bScheduleVideoActive)
				{
					break;
				}

				FOutputFrame* OutputFrameToSchedule = nullptr;

				{
					std::unique_lock<std::mutex> lock(WaitForVideoScheduleMutex);
					WaitForVideoScheduleCondition.wait(lock, [this]() { return !bScheduleVideoActive || InFlightFrames <= 1; });

					VideoFramesToSchedule.Pop(OutputFrameToSchedule);
					
					if (OutputFrameToSchedule)
					{
#if PLATFORM_WINDOWS
						if (ChannelOptions.bUseGPUDMA && TextureTransfer)
						{
							void* DestinationBuffer;
							OutputFrameToSchedule->BlackmagicFrame->GetBytes(&DestinationBuffer);
							TextureTransfer->BeginSync(DestinationBuffer, UE::GPUTextureTransfer::ETransferDirection::GPU_TO_CPU);
						}
#endif //PLATFORM_WINDOWS
						
						ScheduleOutputFrame(OutputFrameToSchedule);
					}
				}
			}

			if (TextureTransfer)
			{
				TextureTransfer->ThreadCleanup();
			}
		}

		void FOutputChannel::AudioThread_ScheduleAudioSamples()
		{
			while (true)
			{
				if (!bScheduleAudioSamplesActive)
				{
					break;
				}

				HRESULT ScheduleAudioResult = S_OK;

				FAudioPacket* PacketToSchedule = nullptr;

				std::unique_lock<std::mutex> lock(WaitForAudioRenderMutex);
				WaitForAudioRenderCondition.wait(lock, [this]() { return !bScheduleAudioSamplesActive || NumBufferedAudioSamples.load() < AudioSamplesWaterLevel * 2; });

				if (bIsPrerollingAudio && NumBufferedAudioSamples < AudioSamplesWaterLevel)
				{
					PacketToSchedule = AudioPacketPool.Acquire();
					if (PacketToSchedule)
					{
						// Send 0's during preroll.
						PacketToSchedule->NumAudioSamples = GetNumAudioSamplesPerFrame();
						PacketToSchedule->AudioBufferSize = PacketToSchedule->NumAudioSamples * (uint8_t)ChannelOptions.NumAudioChannels * GetAudioBitDepth() / 8;
						PacketToSchedule->Clear();
					}
					else
					{
						UE_LOGF(LogBlackmagicCore, Error,"Could not preroll audio for Output device %d", ChannelInfo.DeviceIndex);
					}
				}

				{
					if (!bIsPrerollingAudio && !PacketToSchedule)
					{
						AudioPacketsToSchedule.Pop(PacketToSchedule);
					}

					if (PacketToSchedule)
					{
						uint32_t SamplesWritten = 0;
						ScheduleAudioResult = DeckLinkOutput->ScheduleAudioSamples(PacketToSchedule->AudioBuffer, PacketToSchedule->NumAudioSamples, 0, 0, &SamplesWritten);
						AudioPacketPool.Release(PacketToSchedule);

						uint32_t BufferedSamples = 0;
						DeckLinkOutput->GetBufferedAudioSampleFrameCount(&BufferedSamples);
						NumBufferedAudioSamples.store(BufferedSamples);
#if AUDIO_DEBUGGING
						auto millisec_since_epoch = std::chrono::duration_cast<std::chrono::milliseconds>(std::chrono::system_clock::now().time_since_epoch()).count();
						UE_LOGF(LogBlackmagicCore, Warning,"(%d) Samples Buffered: %d, Expected: %d, Written: %d, PacketsToScheduleSize: %d", millisec_since_epoch, NumBufferedAudioSamples.load(), PacketToSchedule->NumAudioSamples, SamplesWritten, AudioPacketsToSchedule.Size());
#endif
					}
				}

				if (ScheduleAudioResult != S_OK)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Could not schedule audio for Output device %d", ChannelInfo.DeviceIndex);
				}
			}
		}

		bool FOutputChannel::VerifyFormatSupport()
		{
			BMDDisplayMode ActualDisplayMode = bmdModeUnknown;
			BOOL bSupported = FALSE;
			HRESULT Result = DeckLinkOutput->DoesSupportVideoMode(bmdVideoConnectionUnspecified, DisplayMode, PixelFormat, bmdNoVideoOutputConversion, bmdSupportedVideoModeDefault, &ActualDisplayMode, &bSupported);
			if (Result != S_OK || ActualDisplayMode == bmdModeUnknown || !bSupported)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not check if the video mode is supported on device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			IDeckLinkDisplayMode* DeckLinkDisplayMode = nullptr;
			Result = DeckLinkOutput->GetDisplayMode(ActualDisplayMode, &DeckLinkDisplayMode);
			if (Result != S_OK || DeckLinkDisplayMode == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not get the display mode on device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}

			BMDTimeValue Denominator;
			BMDTimeScale Numerator;
			Result = DeckLinkDisplayMode->GetFrameRate(&Denominator, &Numerator);
			if (Result != S_OK || Denominator == 0)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not get a valid frame rate for output on device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			ChannelOptions.FormatInfo.FrameRateNumerator = (uint32_t)Numerator;
			ChannelOptions.FormatInfo.FrameRateDenominator = (uint32_t)Denominator;

			DeckLinkDisplayMode->Release();

			if (!bSupported)
			{
				UE_LOGF(LogBlackmagicCore, Error,"The video format '%d' is not support on device '%d'.", DisplayMode, ChannelInfo.DeviceIndex);
				return false;
			}

			return true;
		}

		bool FOutputChannel::VerifyIfDeviceUsed()
		{
			ReferencePtr<IDeckLinkStatus> DeckLinkStatus;
			HRESULT Result = DeckLink->QueryInterface(IID_IDeckLinkStatus, (void**)&DeckLinkStatus);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Couldn't get the DeckLinkStatus for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			int64_t BusyStatus = 0;
			Result = DeckLinkStatus->GetInt(bmdDeckLinkStatusBusy, &BusyStatus);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Couldn't get the Busy status for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			//Verify if we're already opened as output
			if ((BusyStatus & bmdDevicePlaybackBusy) != 0)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Device %d can't be used because it's already being used as an output.", ChannelInfo.DeviceIndex);
				return false;
			}

			int64_t DuplexStatus = 0;
			Result = DeckLinkAttributes->GetInt(BMDDeckLinkDuplex, &DuplexStatus);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Couldn't get the DuplexMode status for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}


			if (DuplexStatus == ENUM(BMDDuplexMode)::bmdDuplexInactive)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Device %d is inactive.", ChannelInfo.DeviceIndex);
				return false;
			}

			//Device we're on can only do playback or capture at the same time. We can't open it for output if it's the case.
			if (DuplexStatus == ENUM(BMDDuplexMode)::bmdDuplexHalf)
			{
				if ((BusyStatus & bmdDeviceCaptureBusy) != 0)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Device %d can't be used because it's already being used as an input and it can't do output at the same time.", ChannelInfo.DeviceIndex);
					return false;
				}
			}

			return true;
		}

		bool FOutputChannel::InitializeKeyer()
		{
			check(DeckLinkConfiguration);

			if (ChannelOptions.bOutputKey)
			{
				if (!Helpers::Is8Bits(ChannelOptions.PixelFormat))
				{
					UE_LOGF(LogBlackmagicCore, Warning,"Can only key if the pixel format contains alpha.");
				}

				HRESULT Result = DeckLink->QueryInterface(IID_IDeckLinkKeyer, (void**)&Keyer);
				if (Result != S_OK)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Could not obtain the IDeckLinkKeyer interface for device '%d'.", ChannelInfo.DeviceIndex);
					return false;
				}

				const bool bExternalKeyer = true;
				Result = Keyer->Enable(bExternalKeyer);
				if (Result != S_OK)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Could not enable external Keyer for device '%d'. Does the device need special configuration in Blackmagic Desktop Video Setup utility?", ChannelInfo.DeviceIndex);
					return false;
				}

				Result = Keyer->SetLevel(255);
				if (Result != S_OK)
				{
					UE_LOGF(LogBlackmagicCore, Error,"Could not set Keyer level for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
					return false;
				}
			}

			return true;
		}

		bool FOutputChannel::InitializeAudio()
		{
			check(DeckLinkOutput);

			AudioSamplesWaterLevel = NumVideoPrerollFrames * (uint32_t)ChannelOptions.AudioSampleRate / (ChannelOptions.FormatInfo.FrameRateNumerator / ChannelOptions.FormatInfo.FrameRateDenominator) ;

			BMDAudioSampleType SampleType;
			if (ChannelOptions.AudioBitDepth == EAudioBitDepth::Signed_16Bits)
			{
				SampleType = bmdAudioSampleType16bitInteger;
			}
			else if (ChannelOptions.AudioBitDepth == EAudioBitDepth::Signed_32Bits)
			{
				SampleType = bmdAudioSampleType32bitInteger;
			}
			else
			{
				check(false);
				UE_LOGF(LogBlackmagicCore, Error,"Unsupported audio bit depth.");
				return false;
			}

			bScheduleAudioSamplesActive = true;
			ScheduleAudioSamplesThread = std::make_unique<std::thread>(&FOutputChannel::AudioThread_ScheduleAudioSamples, this);
			BlackmagicPlatform::SetThreadPriority_TimeCritical(*ScheduleAudioSamplesThread);

			// @Note: Only 48 kHz is currently supported on current blackmagic devices.
			if (FAILED(DeckLinkOutput->EnableAudioOutput(bmdAudioSampleRate48kHz, SampleType, static_cast<uint32_t>(ChannelOptions.NumAudioChannels), bmdAudioOutputStreamContinuous))
				|| FAILED(DeckLinkOutput->SetAudioCallback(RenderAudioCallback)))
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not enable audio output for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			if (FAILED(DeckLinkOutput->BeginAudioPreroll()))
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not begin audio preroll for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			bAudioOutputStarted = true;

			return true;
		}

		bool FOutputChannel::InitializeVideo()
		{
			check(DeckLinkOutput);

			HRESULT Result = DeckLinkOutput->EnableVideoOutput(DisplayMode, VideoOutputFlags);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not enable video output for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			bVideoOutputStarted = true;
			OutputTime = 0;

			if (ChannelOptions.bScheduleInDifferentThread)
			{
				bScheduleVideoActive = true;
				ScheduleVideoThread = std::make_unique<std::thread>(&FOutputChannel::VideoThread_ScheduleVideo, this);
				BlackmagicPlatform::SetThreadPriority_TimeCritical(*ScheduleVideoThread);	
			}

			//Fill in x frames to kickstart the frame complete callback. One frame in flight is necessary to have the desired frame rate.
			if (!SendPreRollFrames())
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not send preroll frames for device '%d'.", ChannelInfo.DeviceIndex);
				return false;
			}

			// Enable video output
			const BMDTimeValue StartTime = 0;
			const BMDTimeScale Scale = ChannelOptions.FormatInfo.FrameRateNumerator;
			const double PlaybackSpeed = 1.0;
			Result = DeckLinkOutput->StartScheduledPlayback(StartTime, Scale, PlaybackSpeed);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Could not start playback for device '%d' - result = %08x.", ChannelInfo.DeviceIndex, Result);
				return false;
			}


			return true;
		}

		bool FOutputChannel::InitializeWaitForInterlacedOddFieldEvent()
		{
			if (ChannelOptions.FormatInfo.FieldDominance == EFieldDominance::Interlaced)
			{
				auto WaitLambda = [&]()
				{
					//Stall for half a frame if in interlaced mode to trigger the artificial field event
					if (ChannelOptions.FormatInfo.FieldDominance == EFieldDominance::Interlaced)
					{
						while (bWaitForInterlacedOddFieldActive)
						{
							std::unique_lock<std::mutex> lock(*WaitForInterlacedOddFieldMutex);
							WaitForInterlacedOddFieldCondition->wait(lock);

							if (!bWaitForInterlacedOddFieldActive)
							{
								return;
							}

							//Check for re-entrance
							if (!bIsWaitingForInterlaceEvent)
							{
								bIsWaitingForInterlaceEvent = true;
								const float SecondsToWait = ((float)ChannelOptions.FormatInfo.FrameRateDenominator / (float)ChannelOptions.FormatInfo.FrameRateNumerator) * 0.5f;
								Helpers::BlockForAmountOfTime(SecondsToWait);
								bIsWaitingForInterlaceEvent = false;

								std::lock_guard<std::mutex> Lock(CallbackMutex);
								for (FOutputChannel::FListener& Listener : Listeners)
								{
									Listener.Callback->OnInterlacedOddFieldEvent();
								}
							}
						}
					}
				};

				bWaitForInterlacedOddFieldActive = true;
				WaitForInterlacedOddFieldMutex = std::make_unique<std::mutex>();
				WaitForInterlacedOddFieldCondition = std::make_unique<std::condition_variable>();
				WaitForInterlacedOddFieldThread = std::make_unique<std::thread>(WaitLambda);
				BlackmagicPlatform::SetThreadPriority_TimeCritical(*WaitForInterlacedOddFieldThread);
			}

			return true;
		}

		bool FOutputChannel::SendPreRollFrames()
		{
			check(!FrameReadyToWrite.empty());
			if (FrameReadyToWrite.size() < 3)
			{
				UE_LOGF(LogBlackmagicCore, Error,"Can't initiate PreRoll. Need at least 3 frames and only '%zu' are available.", FrameReadyToWrite.size());
				return false;
			}

			//Preroll the system with the last available frame.
			FOutputFrame* Frame = FrameReadyToWrite.back();
			FrameReadyToWrite.erase(--FrameReadyToWrite.end());

			//We're pushing the same frame 2 times just to kickstart the output process
			//We always need a frame in flight for timing purposes
			for (int32_t i = 0; i < NumVideoPrerollFrames; ++i)
			{
				if (!ScheduleOutputFrame(Frame))
				{
					return false;
				}
			}

			if (ChannelOptions.bOutputAudio)
			{
				DeckLinkOutput->EndAudioPreroll();
				bIsPrerollingAudio = false;
			}
			return true;
		}

		void FOutputChannel::EmbedTimecodeInFrame(FOutputFrame* InFrame)
		{
			const FTimecode AdjustedTimecode = Helpers::AdjustTimecodeFromUE(ChannelOptions.FormatInfo, InFrame->Timecode);
			const BMDTimecodeFlags Flags = InFrame->Timecode.bIsDropFrame ? bmdTimecodeIsDropFrame : bmdTimecodeFlagDefault;
			HRESULT Result = InFrame->BlackmagicFrame->SetTimecodeFromComponents(DeviceTimecodeFormat, AdjustedTimecode.Hours, AdjustedTimecode.Minutes, AdjustedTimecode.Seconds, AdjustedTimecode.Frames, Flags);
			if (Result != S_OK)
			{
				UE_LOGF(LogBlackmagicCore, Warning,"Could not set Timecode on frame for Output device %d", ChannelInfo.DeviceIndex);
			}
		}

		FOutputChannel::~FOutputChannel()
		{
			if (NotificationCallback && DeckLinkOutput)
			{
				DeckLinkOutput->SetScheduledFrameCompletionCallback(nullptr);
			}

			if (ScheduleVideoThread)
			{
				bScheduleVideoActive = false;
				WaitForVideoScheduleCondition.notify_one();
				ScheduleVideoThread->join();
				ScheduleVideoThread.reset();
			}

			if (ScheduleAudioSamplesThread)
			{
				bScheduleAudioSamplesActive = false;
				WaitForAudioRenderCondition.notify_one();
				ScheduleAudioSamplesThread->join();
				ScheduleAudioSamplesThread.reset();
			}

			if (DeckLinkOutput)
			{
				// Disable the video and audio output interface
				if (bVideoOutputStarted)
				{
					if (Keyer)
					{
						Keyer->Disable();
					}
					DeckLinkOutput->DisableVideoOutput();
					DeckLinkOutput->StopScheduledPlayback(0, nullptr, 0);

					bVideoOutputStarted = false;
				}

				if (bAudioOutputStarted)
				{
					DeckLinkOutput->DisableAudioOutput();
					bAudioOutputStarted = false;
				}
				
				if (bAudioOutputStarted)
				{
					bAudioOutputStarted = false;
				}

				if (RenderAudioCallback)
				{
					DeckLinkOutput->SetAudioCallback(nullptr);
				}

				if (Allocator)
				{
					DeckLinkOutput->SetVideoOutputFrameMemoryAllocator(nullptr);
					Allocator->Release();
				}
			}

			if (NotificationCallback)
			{
				NotificationCallback->Release();
			}

			if (RenderAudioCallback)
			{
				RenderAudioCallback->Release();
			}
			
			if (Keyer)
			{
				Keyer->Release();
			}

			//Wait for activity on the bus to be over.
			if (DeckLink)
			{
				ReferencePtr<IDeckLinkStatus> DeckLinkStatus;
				HRESULT Result = DeckLink->QueryInterface(IID_IDeckLinkStatus, (void**)&DeckLinkStatus);
				if (Result == S_OK)
				{
					static const std::chrono::milliseconds TimeToWait = std::chrono::milliseconds(500);
					const std::chrono::high_resolution_clock::time_point TargetTime = std::chrono::high_resolution_clock::now() + TimeToWait;

					int64_t BusyStatus = bmdDevicePlaybackBusy;
					while ((BusyStatus & bmdDevicePlaybackBusy) != 0 && std::chrono::high_resolution_clock::now() < TargetTime)
					{
						DeckLinkStatus->GetInt(bmdDeckLinkStatusBusy, &BusyStatus);
					}

					if ((BusyStatus & bmdDevicePlaybackBusy) != 0)
					{
						UE_LOGF(LogBlackmagicCore, Warning,"Could not stop capture activity on device '%d' while closing it.", ChannelInfo.DeviceIndex);
					}
				}
			}

			if (DeckLinkConfiguration)
			{
				DeckLinkConfiguration->Release();
			}

			if (DeckLinkAttributes)
			{
				DeckLinkAttributes->Release();
			}

			if (DeckLinkOutput)
			{
				DeckLinkOutput->Release();
			}

			if (DeckLink)
			{
				DeckLink->Release();
			}

			if (WaitForInterlacedOddFieldThread)
			{
				bWaitForInterlacedOddFieldActive = false;
				WaitForInterlacedOddFieldCondition->notify_one();
				WaitForInterlacedOddFieldThread->join();
				WaitForInterlacedOddFieldThread.reset();
			}

			//Clean the frames that we created
			for (FOutputFrame* FrameIterator : AllFrames)
			{
				delete FrameIterator;
			}
			AllFrames.clear();
		}

		size_t FOutputChannel::NumberOfListeners()
		{
			std::lock_guard<std::mutex> Lock(CallbackMutex);
			return Listeners.size();
		}

		void FOutputChannel::AddListener(const FUniqueIdentifier& InIdentifier, ReferencePtr<IOutputEventCallback> InCallback)
		{
			FListener Listener;
			Listener.Identifier = InIdentifier;
			Listener.Callback = std::move(InCallback);

			std::lock_guard<std::mutex> Lock(CallbackMutex);
			Listeners.push_back(Listener);
		}

		void FOutputChannel::RemoveListener(const FUniqueIdentifier& InIdentifier)
		{
			bool bFound = false;
			FListener FoundListener;

			{
				std::lock_guard<std::mutex> Lock(CallbackMutex);
				auto FoundItt = std::find_if(std::begin(Listeners), std::end(Listeners), [=](const FListener& InOther) { return InOther.Identifier == InIdentifier; });
				if (FoundItt != std::end(Listeners))
				{
					bFound = true;
					FoundListener = *FoundItt;
					Listeners.erase(FoundItt);
				}
			}

			if (bFound)
			{
				FoundListener.Callback->OnShutdownCompleted();
			}
		}

		bool FOutputChannel::SetVideoFrameData(const FFrameDescriptor& InFrame)
		{
			const bool bIsProgressive = ChannelOptions.FormatInfo.FieldDominance != EFieldDominance::Interlaced || ChannelOptions.bOutputInterlaceAsProgressive;

			if (InFrame.VideoBuffer == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"SetVideoFrameData: Can't set the video. The buffer is invalid or the buffer size is not the same as the Blackmagic Video Format for output channel %d.\n", ChannelInfo.DeviceIndex);
				return false;
			}

			FOutputChannel::FOutputFrame* AvailableWritingFrame = FetchAvailableWritingFrame(InFrame);
			if (!AvailableWritingFrame)
			{
				return false;
			}

			void* DestinationBuffer;
			AvailableWritingFrame->BlackmagicFrame->GetBytes(&DestinationBuffer);
			const uint32_t VideoBufferSize = AvailableWritingFrame->BlackmagicFrame->GetRowBytes() * AvailableWritingFrame->BlackmagicFrame->GetHeight();

			const int32_t Stride = AvailableWritingFrame->BlackmagicFrame->GetRowBytes();
			const bool bField1 = AvailableWritingFrame->FrameIdentifier == InFrame.FrameIdentifier;

			if (bIsProgressive)
			{
				// Make sure SSE2 is supported and everything is correctly aligned otherwise fall back on regular memcpy
				// Reduced performance were seen in Linux Vulkan so SSE2Memcpy is disabled for this platform
				if (bIsSSE2Available && IsCorrectlyAlignedForSSE2MemCpy(DestinationBuffer, InFrame.VideoBuffer, VideoBufferSize))
				{
					SSE2MemCpy(DestinationBuffer, InFrame.VideoBuffer, VideoBufferSize);
				}
				else
				{
					memcpy(DestinationBuffer, InFrame.VideoBuffer, VideoBufferSize);
				} 
			}
			else
			{
				// Only write the even or odd frame
				for (int32_t IndexY = bField1 ? 0 : 1; IndexY < InFrame.VideoHeight; IndexY += 2)
				{
					int8_t* Destination = reinterpret_cast<int8_t*>(DestinationBuffer) + (Stride*IndexY);
					int8_t* Source = reinterpret_cast<int8_t*>(InFrame.VideoBuffer) + (Stride*IndexY);
					memcpy(Destination, Source, Stride);
				}
			}


			if (bIsProgressive)
			{
				AvailableWritingFrame->bVideoLineFilled = true;
				AvailableWritingFrame->bVideoF2LineFilled = true;
			}
			else
			{
				if (bField1)
				{
					AvailableWritingFrame->bVideoLineFilled = true;
				}
				else
				{
					AvailableWritingFrame->bVideoF2LineFilled = true;
				}
			}

			AvailableWritingFrame->CopiedVideoBufferSize = VideoBufferSize;

			PushWhenFrameReady(AvailableWritingFrame);

			return true;
		}

		bool FOutputChannel::SetVideoFrameData(FFrameDescriptor_GPUDMA& InFrame)
		{
#if PLATFORM_WINDOWS
			const bool bIsProgressive = ChannelOptions.FormatInfo.FieldDominance != EFieldDominance::Interlaced || ChannelOptions.bOutputInterlaceAsProgressive;

			if (!bIsProgressive)
			{
				UE_LOGF(LogBlackmagicCore, Error,"SetVideoFrameData: Can't set the video, GPUTextureTransfer is not supported with interlaced video.\n");
				return false;
			}

			if (InFrame.RHITexture == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"SetVideoFrameData: Can't set the video. The texture is invalid.\n");
				return false;
			}

			if (!ChannelOptions.bUseGPUDMA || !TextureTransfer)
			{
				UE_LOGF(LogBlackmagicCore, Error,"SetVideoFrameData: Can't set the video. GPU DMA was not setup correctly.\n");
				return false;
			}

			FOutputChannel::FOutputFrame* AvailableWritingFrame = FetchAvailableWritingFrame(InFrame);
			if (!AvailableWritingFrame)
			{
				return false;
			}

			const int32_t VideoHeight = ChannelOptions.FormatInfo.Height;
			const uint32_t Stride = AvailableWritingFrame->BlackmagicFrame->GetRowBytes();
			const uint32_t VideoWidth = ChannelOptions.PixelFormat == EPixelFormat::pf_8Bits ? Stride / 4 : Stride / 16;

			void* DestinationBuffer;
			AvailableWritingFrame->BlackmagicFrame->GetBytes(&DestinationBuffer);

			const uint32_t VideoBufferSize = AvailableWritingFrame->BlackmagicFrame->GetRowBytes() * AvailableWritingFrame->BlackmagicFrame->GetHeight();
			
			if (!bRegisteredDMABuffers)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(BlackmagicOutputChannel::RegisterGPUDirectBuffer);
				for (FOutputFrame* Frame : AllFrames)
				{
					void* Buffer = nullptr;
					if (SUCCEEDED(Frame->BlackmagicFrame->GetBytes(&Buffer)) && TextureTransfer)
					{
						UE::GPUTextureTransfer::FRegisterDMABufferArgs Args;
						Args.Height = VideoHeight;
						Args.Width = VideoWidth;
						Args.Stride = Stride;
						Args.Buffer = Buffer;
						Args.PixelFormat = ChannelOptions.PixelFormat == EPixelFormat::pf_8Bits ? UE::GPUTextureTransfer::EPixelFormat::PF_8Bit : UE::GPUTextureTransfer::EPixelFormat::PF_10Bit;
						TextureTransfer->RegisterBuffer(Args);
					}
				}

				bRegisteredDMABuffers = true;
			}

			if (TextureTransfer)
			{
				UE::GPUTextureTransfer::ETransferDirection Direction = UE::GPUTextureTransfer::ETransferDirection::GPU_TO_CPU;
				
				TRACE_CPUPROFILER_EVENT_SCOPE(BlackmagicOutputChannel::TransferTexture);
				TextureTransfer->TransferTexture(DestinationBuffer, InFrame.RHITexture, Direction);
				AvailableWritingFrame->bVideoLineFilled = true;
				AvailableWritingFrame->bVideoF2LineFilled = true;
				AvailableWritingFrame->CopiedVideoBufferSize = VideoBufferSize;
			}
			PushWhenFrameReady(AvailableWritingFrame);

			return true;
#else
			return false; //Linux not supported yet
#endif // PLATFORM_WINDOWS
		}

		bool FOutputChannel::ShouldCaptureThisFrame(::FTimecode Timecode, uint32 FrameNumber)
		{
			// In interlace, don't capture for an odd frame if we don't find an even frame for it.
			if (ChannelOptions.IsInterlaced() && Timecode.Frames % 2 != 0 && ChannelOptions.bOutputInterlaceOnEvenFrames)
			{
				// Odd frame should always have a matching frame.
				// Find a frame where the even frame is filled but not the odd frame and have the same timecode (only in interlaced)
				for (FOutputFrame* Frame : FrameReadyToWrite)
				{
					if (Frame->FrameIdentifier != FOutputFrame::InvalidFrameIdentifier && Frame->FrameIdentifierF2 == FOutputFrame::InvalidFrameIdentifier)
					{
						if (Frame->FrameIdentifier + 1 == FrameNumber)
						{
							return true;
						}
					}
				}

				if (!FrameReadyToWrite.size())
				{
					UE_LOGF(LogBlackmagicCore, Verbose, "Skipping output because no writable frame is available.");
				}
				else
				{
					for (FOutputFrame* Frame : FrameReadyToWrite)
					{
						UE_LOGF(LogBlackmagicCore, VeryVerbose, "\t F1: %d, F2: %d", Frame->FrameIdentifier, Frame->FrameIdentifierF2);
					}
				}

				// Dont interlace starting with an odd frame.
				return false;
			}
			else
			{
				return true;
			}
		}


		bool FOutputChannel::SetAudioFrameData(const FAudioSamplesDescriptor& InSamples)
		{
			if (InSamples.NumAudioSamples == 0)
			{
				return true;
			}

			if (InSamples.AudioBuffer == nullptr)
			{
				UE_LOGF(LogBlackmagicCore, Error,"SetAudioFrameData: Can't set the audio. The buffer is invalid or the buffer size is not the same as the Blackmagic Audio Format for output channel %d.\n", ChannelInfo.DeviceIndex);
				return false;
			}
			
			if (FAudioPacket* Packet = AudioPacketPool.Acquire())
			{
				memcpy(Packet->AudioBuffer, InSamples.AudioBuffer, InSamples.AudioBufferLength);
				Packet->AudioBufferSize = InSamples.AudioBufferLength;
				Packet->NumAudioSamples = InSamples.NumAudioSamples;

#if AUDIO_DEBUGGING
				auto millisec_since_epoch = std::chrono::duration_cast<std::chrono::milliseconds>(std::chrono::system_clock::now().time_since_epoch()).count();
				std::ofstream myFile("data.pcm", std::ios::out | std::ios::binary | std::ios::app);
				myFile.write((char*)Packet->AudioBuffer, Packet->AudioBufferSize);

				UE_LOGF(LogBlackmagicCore, Error,"SetAudio (%d), (%d:%d) %d samples", millisec_since_epoch, InSamples.Timecode.Seconds, InSamples.Timecode.Frames, Packet->NumAudioSamples);;
#endif
				AudioPacketsToSchedule.Push(Packet);


				return true;
			}

			return false;
		}

		// For Progressive and PSF. Find the frame that have the same identifier. (the image may have been given but the anc and audio is incoming)
		// For Interlaced, there is 2 options. Use Timecode to identify which is the odd and even field. Use FrameIdentifier which is even and odd field.
		BlackmagicDesign::Private::FOutputChannel::FOutputFrame* FOutputChannel::FetchAvailableWritingFrame(const FBaseFrameData& InFrame)
		{
			const bool bIsProgressive = ChannelOptions.FormatInfo.FieldDominance != EFieldDominance::Interlaced || ChannelOptions.bOutputInterlaceAsProgressive;
			return bIsProgressive ? FetchAvailableWritingFrameProgressive(InFrame) : FetchAvailableWritingFrameInterlaced(InFrame);
		}

		BlackmagicDesign::Private::FOutputChannel::FOutputFrame* FOutputChannel::FetchAvailableWritingFrameProgressive(const FBaseFrameData& InFrame)
		{
			FOutputFrame* AvailableWritingFrame = nullptr;
			auto InitializeFrame = [&InFrame](FOutputFrame* OutputFrame) -> FOutputFrame*
			{
				OutputFrame->FrameIdentifier = InFrame.FrameIdentifier;
				OutputFrame->FrameIdentifierF2 = InFrame.FrameIdentifier;
				OutputFrame->Timecode = InFrame.Timecode;
				return OutputFrame;
			};

			std::lock_guard<std::mutex> Lock(FrameLock);

			auto FindFrameFromIdentifier = [this](uint32_t TargetFrameIdentifier) 
			{
				auto MatchingFrameIt = std::find_if(FrameReadyToWrite.begin(), FrameReadyToWrite.end(), [TargetFrameIdentifier] (FOutputFrame* FrameIt) { return FrameIt->FrameIdentifier == TargetFrameIdentifier; });
				return MatchingFrameIt != FrameReadyToWrite.end() ? *MatchingFrameIt : nullptr;
			};


			// Find a matching frame identifier
			if (FOutputFrame* MatchingFrame = FindFrameFromIdentifier(InFrame.FrameIdentifier))
			{
				return MatchingFrame;
			}

			// Find an empty frame
			if (FOutputFrame* EmptyFrame = FindFrameFromIdentifier(FOutputFrame::InvalidFrameIdentifier))
			{
				check(EmptyFrame->FrameIdentifierF2 == FOutputFrame::InvalidFrameIdentifier);
				return InitializeFrame(EmptyFrame);
			}

			// Take the oldest frame not yet written
			if (FrameReadyToWrite.size() > 0)
			{
				std::sort(FrameReadyToWrite.begin(), FrameReadyToWrite.end(), [](FOutputFrame* FrameA, FOutputFrame* FrameB) { return FrameA->FrameIdentifier < FrameB->FrameIdentifier; });

				// This is not thread-safe with but FetchAvailableWritingFrame & IsFrameReadyToBeRead are both in inside a lock.
				// So even if we are currently writing in a buffer, it will be cleared and will not be available to be added to the read list.
				FOutputFrame* OldestFrame = FrameReadyToWrite[0];
				OldestFrame->Clear();
				++LostFrameCounter;
				return InitializeFrame(OldestFrame);
			}

			return nullptr;
		}

		BlackmagicDesign::Private::FOutputChannel::FOutputFrame* FOutputChannel::FetchAvailableWritingFrameInterlaced(const FBaseFrameData& InFrame)
		{
			auto InitializeFrame = [&InFrame](FOutputFrame* Frame) -> FOutputFrame*
			{
				if (Frame->FrameIdentifier == FOutputFrame::InvalidFrameIdentifier)
				{
					Frame->FrameIdentifier = InFrame.FrameIdentifier;
					Frame->Timecode = InFrame.Timecode;
				}
				else
				{
					Frame->FrameIdentifierF2 = InFrame.FrameIdentifier;
					Frame->Timecode2 = InFrame.Timecode;
				}

				return Frame;
			};

			auto FindFrameFromIdentifier = [this](uint32_t TargetFrameIdentifier) 
			{
				auto MatchingFrameIt = std::find_if(FrameReadyToWrite.begin(), FrameReadyToWrite.end(), [TargetFrameIdentifier](FOutputFrame* FrameIt) { return FrameIt->FrameIdentifier == TargetFrameIdentifier; });
				return MatchingFrameIt != FrameReadyToWrite.end() ? *MatchingFrameIt : nullptr;
			};

			std::lock_guard<std::mutex> Lock(FrameLock);

			// Find a matching frame identifier
			for (FOutputFrame* FrameItt : FrameReadyToWrite)
			{
				if (FrameItt->FrameIdentifierF2 == InFrame.FrameIdentifier)
				{
					return InitializeFrame(FrameItt);
				}
			}

			// Find a frame where the even frame is filled but not the odd frame and have the same timecode (only in interlaced)
			for (FOutputFrame* FrameItt : FrameReadyToWrite)
			{
				if (FrameItt->FrameIdentifier != FOutputFrame::InvalidFrameIdentifier && FrameItt->FrameIdentifierF2 == FOutputFrame::InvalidFrameIdentifier)
				{
					if (FrameItt->FrameIdentifier + 1 == InFrame.FrameIdentifier)
					{
						if (ChannelOptions.bOutputInterlacedFieldsTimecodeNeedToMatch)
						{
							if (FrameItt->Timecode == InFrame.Timecode)
							{
								return InitializeFrame(FrameItt);
							}
						}
						else
						{
							return InitializeFrame(FrameItt);
						}
					}
				}
			}

			// Find a new empty frames
			if (!ChannelOptions.bOutputInterlacedFieldsTimecodeNeedToMatch)
			{
				// We don't use this path when using bOutputInterlaceOnEvenFrames, since frames that land on an odd 
				// frame counter in the render thread may actually be even timecode frames.
				if (!ChannelOptions.bOutputInterlaceOnEvenFrames)
				{ 
					if (!InFrame.bEvenFrame)
					{
						// If the incoming frame is interlaced and we didn't find the correspondent even field
						// do not create a new field. Drop it.
						++LostFrameCounter;
						if (ChannelOptions.bLogDropFrames)
						{
							UE_LOGF(LogBlackmagicCore, Warning,"Could not find frame. id %d, total flight %d, available %zu", InFrame.FrameIdentifier, InFlightFrames.load(), FrameReadyToWrite.size());
						}

						return nullptr;
					}
				}
			}

			// Find an empty frame
			if (FOutputFrame* EmptyFrame = FindFrameFromIdentifier(FOutputFrame::InvalidFrameIdentifier))
			{
				check(EmptyFrame->FrameIdentifierF2 == FOutputFrame::InvalidFrameIdentifier);
				return InitializeFrame(EmptyFrame);
			}

			// Take the oldest frame not yet written
			if (FrameReadyToWrite.size() > 0)
			{
				std::sort(FrameReadyToWrite.begin(), FrameReadyToWrite.end(), [](FOutputFrame* FrameA, FOutputFrame* FrameB) { return FrameA->FrameIdentifier < FrameB->FrameIdentifier; });

				// This is not thread-safe with but FetchAvailableWritingFrame & IsFrameReadyToBeRead are both in inside a lock.
				// So even if we are currently writing in a buffer, it will be cleared and will not be available to be added to the read list.
				FOutputFrame* OldestFrame = FrameReadyToWrite[0];
				OldestFrame->Clear();
				++LostFrameCounter;
				return InitializeFrame(OldestFrame);
			}

			return nullptr;
		}

		void FOutputChannel::PushWhenFrameReady(FOutputFrame* InFrame)
		{
			std::lock_guard<std::mutex> Lock(FrameLock);
			if (IsFrameReadyToBeRead(InFrame))
			{
				UE_LOGF(LogBlackmagicCore, Verbose, "Pushing frame %d, FrameReadyToWriteSize = %zu", InFrame->FrameIdentifier, FrameReadyToWrite.size());
				auto FoundItt = std::find(std::begin(FrameReadyToWrite), std::end(FrameReadyToWrite), InFrame);
				if (FoundItt != std::end(FrameReadyToWrite))
				{
					FrameReadyToWrite.erase(FoundItt);

					if (ChannelOptions.bScheduleInDifferentThread)
					{
						VideoFramesToSchedule.Push(InFrame);
						WaitForVideoScheduleCondition.notify_one();
					}
					else
					{
	#if PLATFORM_WINDOWS
						if (ChannelOptions.bUseGPUDMA && TextureTransfer)
						{

							void* DestinationBuffer;
							InFrame->BlackmagicFrame->GetBytes(&DestinationBuffer);

							UE::GPUTextureTransfer::ETransferDirection Direction = UE::GPUTextureTransfer::ETransferDirection::GPU_TO_CPU;
							TextureTransfer->BeginSync(DestinationBuffer, UE::GPUTextureTransfer::ETransferDirection::GPU_TO_CPU);
						}
	#endif //PLATFORM_WINDOWS

						ScheduleOutputFrame(InFrame);
					}
				}
			}
		}

		bool FOutputChannel::IsFrameReadyToBeRead(FOutputFrame* InFrame)
		{
			//@todo Update to support ancillary
			if (InFrame->FrameIdentifier == FOutputFrame::InvalidFrameIdentifier)
			{
				InFrame->Clear();
				return false;
			}
			//if (UseAncillary() && (!InFrame->bAncLineFilled || InFrame->CopiedAncBufferSize == 0))
			//{
			//	return false;
			//}
			//if (UseAncillaryField2() && (!InFrame->bAncF2LineFilled || InFrame->CopiedAncF2BufferSize == 0))
			//{
			//	return false;
			//}
			if (ChannelOptions.bOutputVideo && (!InFrame->bVideoF2LineFilled || !InFrame->bVideoLineFilled || InFrame->CopiedVideoBufferSize == 0))
			{
				return false;
			}
			return true;
		}

		HRESULT FDecklinkVideoFrame::QueryInterface(REFIID RIID, LPVOID* PPV)
		{
			HRESULT Result = S_OK;

			if (PPV == nullptr)
			{
				return E_INVALIDARG;
			}

			if (IsEqualIID(RIID, IID_IUnknown) || IsEqualIID(RIID, IID_IDeckLinkVideoFrame))
			{
				*PPV = static_cast<IDeckLinkMutableVideoFrame*>(this);
				AddRef();
			}
			else if (IsEqualIID(RIID, IID_IDeckLinkVideoFrameMetadataExtensions) && HDRMetadata.bIsAvailable)
			{
				*PPV = static_cast<IDeckLinkVideoFrameMetadataExtensions*>(this);
				AddRef();
			}
			else
			{
				*PPV = nullptr;
				Result = E_NOINTERFACE;
			}

			return Result;
		}

		ULONG FDecklinkVideoFrame::AddRef()
		{
			return ++RefCount;
		}

		ULONG FDecklinkVideoFrame::Release()
		{
			const ULONG NewRefValue = --RefCount;

			if (NewRefValue == 0)
			{
				delete this;
			}

			return NewRefValue;
		}

		BMDFrameFlags FDecklinkVideoFrame::GetFlags(void)
		{
			if (HDRMetadata.bIsAvailable)
			{
				return WrappedVideoFrame->GetFlags() | bmdFrameContainsHDRMetadata;
			}
			return WrappedVideoFrame->GetFlags();
		}

		long FDecklinkVideoFrame::GetWidth(void)
		{
			return WrappedVideoFrame->GetWidth();
		}

		long FDecklinkVideoFrame::GetHeight(void)
		{
			return WrappedVideoFrame->GetHeight();
		}

		long FDecklinkVideoFrame::GetRowBytes(void)
		{
			return WrappedVideoFrame->GetRowBytes();
		}

		BMDPixelFormat FDecklinkVideoFrame::GetPixelFormat(void)
		{
			return WrappedVideoFrame->GetPixelFormat();
		}

		HRESULT FDecklinkVideoFrame::GetBytes(void** Buffer)
		{
			return WrappedVideoFrame->GetBytes(Buffer);
		}

		HRESULT FDecklinkVideoFrame::GetTimecode(BMDTimecodeFormat Format, IDeckLinkTimecode** Timecode)
		{
			return WrappedVideoFrame->GetTimecode(Format, Timecode);
		}

		HRESULT FDecklinkVideoFrame::GetAncillaryData(IDeckLinkVideoFrameAncillary** Ancillary)
		{
			return WrappedVideoFrame->GetAncillaryData(Ancillary);
		}

		HRESULT FDecklinkVideoFrame::SetFlags(BMDFrameFlags NewFlags)
		{
			return WrappedVideoFrame->SetFlags(NewFlags);
		}

		HRESULT FDecklinkVideoFrame::SetTimecode(BMDTimecodeFormat Format, IDeckLinkTimecode* Timecode)
		{
			return WrappedVideoFrame->SetTimecode(Format, Timecode);
		}

		HRESULT FDecklinkVideoFrame::SetTimecodeFromComponents(BMDTimecodeFormat Format, unsigned char Hours, unsigned char Minutes, unsigned char Seconds, unsigned char Frames, BMDTimecodeFlags Flags)
		{
			return WrappedVideoFrame->SetTimecodeFromComponents(Format, Hours, Minutes, Seconds, Frames, Flags);
		}

		HRESULT FDecklinkVideoFrame::SetAncillaryData(IDeckLinkVideoFrameAncillary* Ancillary)
		{
			return WrappedVideoFrame->SetAncillaryData(Ancillary);
		}

		HRESULT FDecklinkVideoFrame::SetTimecodeUserBits(BMDTimecodeFormat Format, BMDTimecodeUserBits UserBits)
		{
			return WrappedVideoFrame->SetTimecodeUserBits(Format, UserBits);
		}

		HRESULT FDecklinkVideoFrame::GetInt(BMDDeckLinkFrameMetadataID MetadataId, LONGLONG* Value)
		{
			HRESULT Result = S_OK;

			switch (MetadataId)
			{
			case bmdDeckLinkFrameMetadataHDRElectroOpticalTransferFunc:
			{
				*Value = static_cast<int64>(HDRMetadata.EOTF);
				break;
			}
			case bmdDeckLinkFrameMetadataColorspace:
			{
				switch (HDRMetadata.ColorSpace)
				{
				case EHDRMetaDataColorspace::Rec601:
					*Value = bmdColorspaceRec601;
					break;
				case EHDRMetaDataColorspace::Rec709:
					*Value = bmdColorspaceRec709;
					break;
				case EHDRMetaDataColorspace::Rec2020:
					*Value = bmdColorspaceRec2020;
					break;
				default:
					if (IsHDRLoggingOK())
					{
						UE_LOGF(LogBlackmagicCore, Warning, "Invalid color space detected.");
					}
						
					*Value = bmdColorspaceRec709;
					break;
				}
				break;
			}
			default:
				Result = E_INVALIDARG;
				break;
			}

			return Result;
		}

		HRESULT FDecklinkVideoFrame::GetFloat(BMDDeckLinkFrameMetadataID MetadataId, double* Value)
		{
			HRESULT Result = S_OK;

			switch (MetadataId)
			{
			case bmdDeckLinkFrameMetadataHDRDisplayPrimariesRedX:
				*Value = HDRMetadata.DisplayPrimariesRedX;
				break;
			case bmdDeckLinkFrameMetadataHDRDisplayPrimariesRedY:
				*Value = HDRMetadata.DisplayPrimariesRedY;
				break;
			case bmdDeckLinkFrameMetadataHDRDisplayPrimariesGreenX:
				*Value = HDRMetadata.DisplayPrimariesGreenX;
				break;
			case bmdDeckLinkFrameMetadataHDRDisplayPrimariesGreenY:
				*Value = HDRMetadata.DisplayPrimariesGreenY;
				break;
			case bmdDeckLinkFrameMetadataHDRDisplayPrimariesBlueX:
				*Value = HDRMetadata.DisplayPrimariesBlueX;
				break;
			case bmdDeckLinkFrameMetadataHDRDisplayPrimariesBlueY:
				*Value = HDRMetadata.DisplayPrimariesBlueY;
				break;
			case bmdDeckLinkFrameMetadataHDRWhitePointX:
				*Value = HDRMetadata.WhitePointX;
				break;
			case bmdDeckLinkFrameMetadataHDRWhitePointY:
                *Value = HDRMetadata.WhitePointY;
                break;
			case bmdDeckLinkFrameMetadataHDRMaxDisplayMasteringLuminance:
				*Value = HDRMetadata.MaxDisplayLuminance;
				break;
			case bmdDeckLinkFrameMetadataHDRMinDisplayMasteringLuminance:
				*Value = HDRMetadata.MinDisplayLuminance;
				break;
			case bmdDeckLinkFrameMetadataHDRMaximumContentLightLevel:
				*Value = HDRMetadata.MaxContentLightLevel;
				break;
			case bmdDeckLinkFrameMetadataHDRMaximumFrameAverageLightLevel:
				*Value = HDRMetadata.MaxFrameAverageLightLevel;
				break;
			default:
				Result = E_INVALIDARG;
				break;
			}
			return Result;
		}

		HRESULT FDecklinkVideoFrame::GetFlag(BMDDeckLinkFrameMetadataID MetadataId, BOOL* Value)
		{
			return E_INVALIDARG;
		}

		HRESULT FDecklinkVideoFrame::GetString(BMDDeckLinkFrameMetadataID MetadataId, BSTR* Value)
		{
			return E_INVALIDARG;
		}

		HRESULT FDecklinkVideoFrame::GetBytes(BMDDeckLinkFrameMetadataID MetadataId, void* Buffer, unsigned int* BufferSize)
		{
			return E_INVALIDARG;
		}

		bool FDecklinkVideoFrame::IsHDRLoggingOK()
		{
			// Reset log count after a few seconds.
            if (FPlatformTime::Seconds() > LastHDRLogResetTime)
            {
            	HDRLogCount = 0;
            }
            
            bool bIsOK = HDRLogCount < 30;

            if (bIsOK)
            {
            	HDRLogCount++;
            	LastHDRLogResetTime = FPlatformTime::Seconds() + 2;
            }

            return bIsOK;
		}
	}
};

CustomAllocator::CustomAllocator()
	: RefCount(1)
	, AllocatedSize(0)
	, AllocatedBuffer(nullptr)
{
}

CustomAllocator::~CustomAllocator()
{
}

HRESULT CustomAllocator::QueryInterface(REFIID iid, LPVOID* ppv)
{
	return E_NOTIMPL;
}

ULONG CustomAllocator::AddRef(void)
{
	return ++RefCount;
}

ULONG CustomAllocator::Release(void)
{
	RefCount--;
	if (RefCount == 0)
	{
		delete this;
		return 0;
	}

	return RefCount;
}

HRESULT CustomAllocator::AllocateBuffer(unsigned int bufferSize, void** allocatedBuffer)
{
	*allocatedBuffer = BlackmagicPlatform::Allocate(bufferSize);
	AllocatedSize = bufferSize;

	if (!*allocatedBuffer)
	{
		return E_OUTOFMEMORY;
	}

	return S_OK;
}

HRESULT CustomAllocator::ReleaseBuffer(void* buffer)
{
	BlackmagicPlatform::Free(buffer, AllocatedSize);
	return S_OK;
}

HRESULT CustomAllocator::Commit()
{
	return S_OK;
}

HRESULT CustomAllocator::Decommit()
{
	BlackmagicPlatform::Free(AllocatedBuffer, AllocatedSize);
	return S_OK;
}
