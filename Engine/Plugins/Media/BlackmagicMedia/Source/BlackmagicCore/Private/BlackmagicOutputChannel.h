// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.h"
#include "GPUTextureTransferModule.h"
#include "ThreadSafeQueue.h"
#include "ThreadSafeObjectPool.h"

#include <atomic>

class CustomAllocator : public IDeckLinkMemoryAllocator
{
public:
	CustomAllocator();
	virtual ~CustomAllocator();

	// IUnknown methods
	virtual HRESULT STDMETHODCALLTYPE	QueryInterface(REFIID iid, LPVOID* ppv);
	virtual ULONG STDMETHODCALLTYPE		AddRef(void);
	virtual ULONG STDMETHODCALLTYPE		Release(void);

	// IDeckLinkMemoryAllocator methods
	virtual HRESULT STDMETHODCALLTYPE	AllocateBuffer(unsigned int bufferSize, void** allocatedBuffer);
	virtual HRESULT STDMETHODCALLTYPE	ReleaseBuffer(void* buffer);
	virtual HRESULT STDMETHODCALLTYPE	Commit();
	virtual HRESULT STDMETHODCALLTYPE	Decommit();

private:
	ULONG RefCount;
	uint32_t AllocatedSize;
	void* AllocatedBuffer;
};


namespace BlackmagicDesign
{
	namespace Private
	{
		struct FAudioPacket : public Blackmagic::Private::IPoolable
		{
			// Assuming lowest framerate of 14.98 fps, Sample rate of 48 KHZ, 32 bit depth and 16 Channels
			// 16 * (32/8) * 48000 / 14.98 = 201 kB
			const uint32_t BLACKMAGIC_AUDIOSIZE_MAX = (201 * 1024);

			uint8_t* AudioBuffer;
			uint32_t NumAudioSamples = 0;
			uint32_t AudioBufferSize = 0;

			FAudioPacket()
			{
				AudioBuffer = (uint8_t*)malloc(BLACKMAGIC_AUDIOSIZE_MAX);
			}

			virtual ~FAudioPacket()
			{
				if (AudioBuffer)
				{
					free(AudioBuffer);
				}
			}

			virtual void Clear()
			{
				if (AudioBuffer && AudioBufferSize != 0)
				{
					memset(AudioBuffer, 0, AudioBufferSize);
				}
			}
		};

		class FOutputChannel;

		class FOutputChannelNotificationCallback : public IDeckLinkVideoOutputCallback
		{
		public:
			FOutputChannelNotificationCallback(FOutputChannel* InOwner, IDeckLinkOutput* InDeckLinkOutput);
			virtual ~FOutputChannelNotificationCallback() = default;

			HRESULT STDMETHODCALLTYPE QueryInterface(REFIID iid, LPVOID *ppv);

			ULONG STDMETHODCALLTYPE AddRef();
			ULONG STDMETHODCALLTYPE Release();

			HRESULT STDMETHODCALLTYPE ScheduledFrameCompleted(IDeckLinkVideoFrame* InCompletedFrame, BMDOutputFrameCompletionResult InResult);
			HRESULT STDMETHODCALLTYPE ScheduledPlaybackHasStopped();

		private:
			FOutputChannel* OutputChannel;
			IDeckLinkOutput* DeckLinkOutput;
			std::atomic_char32_t RefCount;
		};

		class FOutputChannelRenderAudioCallback : public IDeckLinkAudioOutputCallback
		{
		public:
			FOutputChannelRenderAudioCallback(FOutputChannel* InOwner, IDeckLinkOutput* InDeckLinkOutput);
			virtual ~FOutputChannelRenderAudioCallback() {}

			HRESULT STDMETHODCALLTYPE RenderAudioSamples(BOOL Preroll);

			HRESULT STDMETHODCALLTYPE QueryInterface(REFIID iid, LPVOID* ppv);

			ULONG STDMETHODCALLTYPE AddRef();
			ULONG STDMETHODCALLTYPE Release();
		
		private:
			FOutputChannel* OutputChannel;
			IDeckLinkOutput* DeckLinkOutput;
			std::atomic_char32_t RefCount;
		};

		/**
		 * Wrapper class around IDeckLinkMutableVideoFrame that also extends IDeckLinkVideoFrameMetadataExtensions in order to provide HDR metadata to the SDI feed.
		 */
		class FDecklinkVideoFrame : public IDeckLinkMutableVideoFrame, public IDeckLinkVideoFrameMetadataExtensions
		{
		public:
			FDecklinkVideoFrame(IDeckLinkMutableVideoFrame* InWrappedVideoFrame, FHDRMetaData InHDRMetadata)
				: RefCount(1)
				, WrappedVideoFrame(InWrappedVideoFrame)
				, HDRMetadata(MoveTemp(InHDRMetadata))
			{
			}

			virtual ~FDecklinkVideoFrame() = default;

			//~ Begin IUnknown interface
			virtual HRESULT STDMETHODCALLTYPE QueryInterface(REFIID RIID, LPVOID* PPV) override;
			virtual ULONG STDMETHODCALLTYPE AddRef() override;
			virtual ULONG STDMETHODCALLTYPE Release() override;
			//~ End IUnknown interface

			//~ Begin IDeckLinkVideoFrame Interface
			virtual BMDFrameFlags STDMETHODCALLTYPE GetFlags(void) override;
			virtual long STDMETHODCALLTYPE GetWidth(void) override;
			virtual long STDMETHODCALLTYPE GetHeight(void) override;
			virtual long STDMETHODCALLTYPE GetRowBytes(void) override;
			virtual BMDPixelFormat STDMETHODCALLTYPE GetPixelFormat(void) override;
			virtual HRESULT STDMETHODCALLTYPE GetBytes(/* [out] */ void** Buffer) override;
			virtual HRESULT STDMETHODCALLTYPE GetTimecode(/* [in] */ BMDTimecodeFormat Format, /* [out] */ IDeckLinkTimecode** Timecode) override;
			virtual HRESULT STDMETHODCALLTYPE GetAncillaryData(/* [out] */ IDeckLinkVideoFrameAncillary** ancillary) override;
			//~ End IDeckLinkVideoFrame Interface

			//~ Begin IDeckLinkMutableVideoFrame Interface
			virtual HRESULT STDMETHODCALLTYPE SetFlags(/* [in] */ BMDFrameFlags NewFlags) override;
			virtual HRESULT STDMETHODCALLTYPE SetTimecode(/* [in] */ BMDTimecodeFormat Format, /* [in] */ IDeckLinkTimecode* Timecode) override;
			virtual HRESULT STDMETHODCALLTYPE SetTimecodeFromComponents(/* [in] */ BMDTimecodeFormat Format, /* [in] */ unsigned char Hours, /* [in] */ unsigned char Minutes, /* [in] */ unsigned char Seconds, /* [in] */ unsigned char Frames, /* [in] */ BMDTimecodeFlags Flags) override;
			virtual HRESULT STDMETHODCALLTYPE SetAncillaryData(/* [in] */ IDeckLinkVideoFrameAncillary* Ancillary) override; 
			virtual HRESULT STDMETHODCALLTYPE SetTimecodeUserBits(/* [in] */ BMDTimecodeFormat Format, /* [in] */ BMDTimecodeUserBits UserBits) override;
			//~ End IDeckLinkMutableVideoFrame Interface

			//~ Begin IDeckLinkVideoFrameMetadataExtensions interface
			virtual HRESULT STDMETHODCALLTYPE GetInt(BMDDeckLinkFrameMetadataID MetadataId, LONGLONG* Value) override;
			virtual HRESULT STDMETHODCALLTYPE GetFloat(BMDDeckLinkFrameMetadataID MetadataId, double* Value) override;
			virtual HRESULT STDMETHODCALLTYPE GetFlag(BMDDeckLinkFrameMetadataID MetadataId, BOOL* Value) override;
			virtual HRESULT STDMETHODCALLTYPE GetString(BMDDeckLinkFrameMetadataID MetadataId, BSTR* Value) override;
			virtual HRESULT STDMETHODCALLTYPE GetBytes(BMDDeckLinkFrameMetadataID MetadataId, void* Buffer, unsigned int* BufferSize) override;
			//~ End IDeckLinkVideoFrameMetadataExtensions interface
			
		private:
			/** Utility function to throttle HDR warning logs. */
			bool IsHDRLoggingOK();

		private:
			/** Used to destroy object at 0 refs. */
			ULONG RefCount;
			/** Wrapped video frame. */
			ReferencePtr<IDeckLinkMutableVideoFrame> WrappedVideoFrame;
			/** Holds HDR Metadata. */
			FHDRMetaData HDRMetadata;
			/** Used for throttling HDR logs. */
			int32_t HDRLogCount;
			/** Used for keeping track of the last HDR logging reset time.  */
			double LastHDRLogResetTime = 0.0;
		};

		class FOutputChannel
		{
		private:
			struct FListener
			{
				FUniqueIdentifier Identifier;
				ReferencePtr<IOutputEventCallback> Callback;
			};

			struct FOutputFrame;

		public:
			FOutputChannel(const FChannelInfo& InChannelInfo, const FOutputChannelOptions InChannelOptions);
			~FOutputChannel();
			FOutputChannel(const FOutputChannel&) = delete;
			FOutputChannel& operator=(const FOutputChannel&) = delete;

			bool Initialize();

			const FChannelInfo& GetChannelInfo() const { return ChannelInfo; }

			bool IsCompatible(const FChannelInfo& InChannelInfo, const FOutputChannelOptions& InChannelOptions) const { return true; }

			size_t NumberOfListeners();
			void AddListener(const FUniqueIdentifier& InIdentifier, ReferencePtr<IOutputEventCallback> InCallback);
			void RemoveListener(const FUniqueIdentifier& InIdentifier);

			bool SetVideoFrameData(const FFrameDescriptor& InFrame);
			bool SetVideoFrameData(FFrameDescriptor_GPUDMA& InFrame);

			// Return whether to capture or discard a frame based on its timecode and frame number.
			bool ShouldCaptureThisFrame(::FTimecode Timecode, uint32 FrameNumber);

			bool SetAudioFrameData(const FAudioSamplesDescriptor& InSamples);

			void OnScheduledFrameCompleted(IDeckLinkVideoFrame* InCompletedFrame, BMDOutputFrameCompletionResult InResult);
			bool OnRenderAudio(BOOL bPreRoll);
			void UnregisterBuffers();

		private:

			bool InitializeInterfaces();
			bool InitializeLinkConfiguration();
			bool InitializeFrameBuffer();
			bool VerifyFormatSupport();
			bool VerifyIfDeviceUsed();
			bool InitializeDeckLink();
			bool InitializeKeyer();
			bool InitializeAudio();
			bool InitializeVideo();
			bool InitializeWaitForInterlacedOddFieldEvent();

			bool SendPreRollFrames();
			void EmbedTimecodeInFrame(FOutputFrame* InFrame);

			

			//Frame management
			FOutputFrame* FetchAvailableWritingFrame(const FBaseFrameData& InFrame);
			FOutputFrame* FetchAvailableWritingFrameInterlaced(const FBaseFrameData& InFrame);
			FOutputFrame* FetchAvailableWritingFrameProgressive(const FBaseFrameData& InFrame);
			void PushWhenFrameReady(FOutputFrame* InFrame);
			bool IsFrameReadyToBeRead(FOutputFrame* InFrame);
			bool ScheduleOutputFrame(FOutputFrame* InFrame);
			void AdvanceTime();

			uint32_t GetNumAudioSamplesPerFrame();
			uint8_t GetAudioBitDepth();

			void VideoThread_ScheduleVideo();
			void AudioThread_ScheduleAudioSamples();

			uint8_t GetBytesPerPixel() const;

		public:
			bool bIsDebugAlive;

		private:
			friend FOutputChannelNotificationCallback;


			FChannelInfo ChannelInfo;
			FOutputChannelOptions ChannelOptions;
			IDeckLink* DeckLink;
			IDeckLinkProfileAttributes* DeckLinkAttributes;
			IDeckLinkOutput* DeckLinkOutput;
			IDeckLinkConfiguration* DeckLinkConfiguration;
			IDeckLinkKeyer* Keyer;
			FOutputChannelNotificationCallback* NotificationCallback;
			FOutputChannelRenderAudioCallback* RenderAudioCallback;

			bool bAudioOutputStarted;
			bool bVideoOutputStarted; 

			std::atomic<BMDTimeValue> OutputTime;

			BMDVideoOutputFlags VideoOutputFlags;
			BMDPixelFormat PixelFormat;
			BMDDisplayMode DisplayMode;

			bool bOutputTimecode;
			BMDTimecodeFormat DeviceTimecodeFormat;

			std::vector<FListener> Listeners;
			std::mutex CallbackMutex;
			std::mutex AllocatorMutex;

			std::unique_ptr<std::thread> ScheduleVideoThread;
			std::unique_ptr<std::thread> ScheduleAudioSamplesThread;
			std::unique_ptr<std::thread> WaitForInterlacedOddFieldThread;
			std::unique_ptr<std::mutex> WaitForInterlacedOddFieldMutex;
			std::unique_ptr<std::condition_variable> WaitForInterlacedOddFieldCondition;

			volatile bool bWaitForInterlacedOddFieldActive;

			volatile bool bScheduleAudioSamplesActive;
			volatile bool bScheduleVideoActive;

			std::mutex WaitForVideoScheduleMutex;
			std::mutex WaitForAudioRenderMutex;
			std::condition_variable WaitForVideoScheduleCondition;
			std::condition_variable WaitForAudioRenderCondition;

			std::atomic<uint32_t> InFlightFrames;
			std::atomic<uint32_t> LostFrameCounter;
			uint32_t DroppedFrameCounter;
			uint32_t QueuedDroppedFrameCount;

			struct FOutputFrame
			{
				FOutputFrame();
				~FOutputFrame();
				FOutputFrame(const FOutputFrame&) = delete;
				FOutputFrame& operator=(const FOutputFrame&) = delete;

				static const uint32_t InvalidFrameIdentifier;

				ReferencePtr<FDecklinkVideoFrame> BlackmagicFrame;
				uint8_t* AudioBuffer;
				uint32_t NumAudioSamples;

				FTimecode Timecode;
				FTimecode Timecode2;

				uint32_t CopiedVideoBufferSize;
				uint32_t CopiedAudioBufferSize;
				uint32_t FrameIdentifier;
				uint32_t FrameIdentifierF2;

				std::atomic<uint32_t> InFlightCount;

				bool bVideoLineFilled;
				bool bVideoF2LineFilled; // used in interlaced
				bool bAudioLineFilled;

				bool bVideoScheduled;
				bool bAudioScheduled;

				void Clear();
			};

			int32_t AudioBufferSize = 0;

			std::mutex FrameLock;
			std::mutex ScheduleLock;
			std::vector<FOutputFrame*> AllFrames;
			std::vector<FOutputFrame*> FrameReadyToWrite; // Ready to be used by the Unreal Engine

			// Pointer to the last sent frame to be reused when no new frames are ready to be sent
			std::atomic<FOutputFrame*> LastScheduledFrame;

			static constexpr uint8_t NumVideoPrerollFrames = 2;
			std::atomic<uint32_t> NumBufferedAudioSamples;
			uint32_t AudioSamplesWaterLevel = 0;

			CustomAllocator* Allocator = nullptr;

			volatile bool bIsWaitingForInterlaceEvent;

			Blackmagic::Private::ThreadSafeQueue<FOutputFrame*> VideoFramesToSchedule;
			Blackmagic::Private::ThreadSafeQueue<FAudioPacket*> AudioPacketsToSchedule;
			Blackmagic::Private::ThreadSafeObjectPool<FAudioPacket> AudioPacketPool;

			UE::GPUTextureTransfer::TextureTransferPtr TextureTransfer;

			bool bIsSSE2Available;
			bool bIsFieldAEven;
			bool bIsFirstFetch;
			bool bIsPrerollingAudio;
			bool bPlaybackStopped;

			bool bRegisteredDMABuffers = false;
		};
	}
}
