// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicVideoFormats.h"

#include "Common.h"

#include "BlackmagicHelper.h"

#include <algorithm>

namespace BlackmagicDesign
{
	namespace Private
	{
		/* VideoFormatsScanner implementation
		*****************************************************************************/
		VideoFormatsScanner::VideoFormatsScanner(int32_t InDeviceId, bool bForOutput)
		{
			IDeckLink* DesiredDeckLink = nullptr;
			IDeckLinkIterator* DeckLinkIterator = BlackmagicPlatform::CreateDeckLinkIterator();
			if (DeckLinkIterator)
			{
				// Obtain the required DeckLink device
				IDeckLink* DeckLinkItt = nullptr;
				int32_t Index = 1;
				for (; (DeckLinkIterator->Next(&DeckLinkItt) == S_OK); ++Index)
				{
					if (Index == InDeviceId)
					{
						DesiredDeckLink = DeckLinkItt;
						break;
					}
					else
					{
						DeckLinkItt->Release();
					}
				}

				BlackmagicPlatform::DestroyDeckLinkIterator(DeckLinkIterator);
			}

			if (DesiredDeckLink != nullptr)
			{
				if (!bForOutput)
				{
					IDeckLinkInput* DeckLinkInput = nullptr;
					HRESULT Result = DesiredDeckLink->QueryInterface(IID_IDeckLinkInput, (void**)&DeckLinkInput);
					if (Result == S_OK && DeckLinkInput != nullptr)
					{
						IDeckLinkDisplayModeIterator* ModeIterator = nullptr;
						Result = DeckLinkInput->GetDisplayModeIterator(&ModeIterator);
						if (Result == S_OK && ModeIterator != nullptr)
						{
							IDeckLinkDisplayMode* Mode = nullptr;
							while (ModeIterator->Next(&Mode) == S_OK)
							{
								BOOL bSupported = FALSE;
								const BMDPixelFormat PixelFormat = ENUM(BMDPixelFormat)::bmdFormat8BitYUV; //@todo
								Result = DeckLinkInput->DoesSupportVideoMode(bmdVideoConnectionUnspecified, Mode->GetDisplayMode(), PixelFormat, bmdNoVideoInputConversion, bmdSupportedVideoModeDefault, nullptr, &bSupported);
								if (Result == S_OK && bSupported)
								{
									FormatList.emplace_back(GetVideoFormat(Mode));
								}

								Mode->Release();
							}

							ModeIterator->Release();
						}

						DeckLinkInput->Release();
					}
				}
				else
				{
					IDeckLinkOutput* DeckLinkOutput = nullptr;
					HRESULT Result = DesiredDeckLink->QueryInterface(IID_IDeckLinkOutput, (void**)&DeckLinkOutput);
					if (Result == S_OK && DeckLinkOutput != nullptr)
					{
						IDeckLinkDisplayModeIterator* ModeIterator = nullptr;
						Result = DeckLinkOutput->GetDisplayModeIterator(&ModeIterator);
						if (Result == S_OK && ModeIterator != nullptr)
						{
							IDeckLinkDisplayMode* Mode = nullptr;
							while (ModeIterator->Next(&Mode) == S_OK)
							{
								BMDDisplayMode ActualDisplayMode = bmdModeUnknown;
								BOOL bSupported = FALSE;
								const BMDPixelFormat PixelFormat = ENUM(BMDPixelFormat)::bmdFormat8BitYUV; //@todo
								Result = DeckLinkOutput->DoesSupportVideoMode(bmdVideoConnectionUnspecified, Mode->GetDisplayMode(), PixelFormat, bmdNoVideoOutputConversion, bmdSupportedVideoModeDefault, &ActualDisplayMode, &bSupported);
								if (Result == S_OK && bSupported)
								{
									FormatList.emplace_back(GetVideoFormat(Mode));
								}

								Mode->Release();
							}

							ModeIterator->Release();
						}

						DeckLinkOutput->Release();
					}
				}

				DesiredDeckLink->Release();
			}
		}

		BlackmagicVideoFormats::VideoFormatDescriptor VideoFormatsScanner::GetVideoFormat(IDeckLinkDisplayMode* InBlackmagicVideoMode)
		{
			if (InBlackmagicVideoMode == nullptr)
			{
				return BlackmagicVideoFormats::VideoFormatDescriptor();
			}

			BlackmagicVideoFormats::VideoFormatDescriptor NewFormat;
			NewFormat.VideoFormatIndex = InBlackmagicVideoMode->GetDisplayMode();

			BMDTimeValue Scale = 0;
			BMDTimeScale FrameDuration = 0;
			const HRESULT Result = InBlackmagicVideoMode->GetFrameRate(&Scale, &FrameDuration);
			check(Result == S_OK);
			NewFormat.FrameRateNumerator = (uint32_t)FrameDuration;
			NewFormat.FrameRateDenominator = (uint32_t)Scale;

			NewFormat.ResolutionWidth = InBlackmagicVideoMode->GetWidth();
			NewFormat.ResolutionHeight = InBlackmagicVideoMode->GetHeight();
			NewFormat.bIsProgressiveStandard = InBlackmagicVideoMode->GetFieldDominance() == bmdProgressiveFrame;
			NewFormat.bIsInterlacedStandard = InBlackmagicVideoMode->GetFieldDominance() == bmdUpperFieldFirst || InBlackmagicVideoMode->GetFieldDominance() == bmdLowerFieldFirst;
			NewFormat.bIsPsfStandard = InBlackmagicVideoMode->GetFieldDominance() == bmdProgressiveSegmentedFrame;
			NewFormat.bIsSD = Helpers::IsSDFormat(InBlackmagicVideoMode->GetDisplayMode());
			NewFormat.bIsHD = Helpers::IsHDFormat(InBlackmagicVideoMode->GetDisplayMode());
			NewFormat.bIs2K = Helpers::Is2kFormat(InBlackmagicVideoMode->GetDisplayMode());
			NewFormat.bIs4K = Helpers::Is4kFormat(InBlackmagicVideoMode->GetDisplayMode());
			NewFormat.bIs8K = Helpers::Is8kFormat(InBlackmagicVideoMode->GetDisplayMode());

			NewFormat.bIsValid = true;
			return NewFormat;
		}
	}
}
