// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.h"
#include "BlackmagicLib.h"
#include <vector>


namespace BlackmagicDesign
{
	namespace Private
	{
		/* VideoFormatsScanner definition
		*****************************************************************************/
		class VideoFormatsScanner
		{
		public:
			VideoFormatsScanner(int32_t InDeviceId, bool bForOutput);

			static BlackmagicDesign::BlackmagicVideoFormats::VideoFormatDescriptor GetVideoFormat(IDeckLinkDisplayMode* InBlackmagicVideoMode);

			std::vector<BlackmagicDesign::BlackmagicVideoFormats::VideoFormatDescriptor> FormatList;
		};
	}
}
