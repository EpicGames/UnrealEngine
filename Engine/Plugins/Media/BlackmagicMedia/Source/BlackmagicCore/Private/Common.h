// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "assert.h"

#include <algorithm>
#include <atomic>
#include <memory>
#include <mutex>
#include <string>
#include <thread>
#include <vector>
#include <chrono>

#if PLATFORM_WINDOWS

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

THIRD_PARTY_INCLUDES_START
	#include "Platform/GenericPlatform.h"
	#include <tchar.h>
	#include <comdef.h>
	#include "DeckLinkAPI_h.h"
	#include "BlackmagicLib.h"
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

#ifndef FALSE
#define FALSE 0
#endif

#ifndef TRUE
#define TRUE 1
#endif


#else
#include "DeckLinkAPI.h"
#include "Platform/GenericPlatform.h"
#include "BlackmagicLib.h"
#include "LinuxCOM.h"

#ifndef FALSE
#define FALSE false
#endif

#ifndef TRUE
#define TRUE true
#endif

#endif

