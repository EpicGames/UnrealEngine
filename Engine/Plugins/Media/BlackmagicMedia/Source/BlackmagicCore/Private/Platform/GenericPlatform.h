// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
	#include "Win/WinPlatform.h"
#elif PLATFORM_LINUX
	#include "Linux/LinuxPlatform.h"
#else
	#error "The platform is not specified or not defined"
#endif
