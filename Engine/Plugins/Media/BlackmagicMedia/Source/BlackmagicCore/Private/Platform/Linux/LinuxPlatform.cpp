// Copyright Epic Games, Inc. All Rights Reserved.


#include "LinuxPlatform.h"

#if PLATFORM_LINUX

#include "Containers/StringConv.h"
#include "Misc/CString.h"

#include "Common.h"
#include <pthread.h>
#include <sys/mman.h>

bool IsEqualIID(REFIID LHS, REFIID RHS)
{
	return memcmp(&LHS, &RHS, sizeof(REFIID)) == 0;
}

namespace BlackmagicPlatform
{

	bool InitializeAPI()
	{
		return true;
	}

	void ReleaseAPI()
	{
	}

	std::string GetName(IDeckLinkDisplayMode* DisplayMode)
	{
		if (DisplayMode)
		{
			const char* Str;
			DisplayMode->GetName(&Str);
			std::string StandardStr(Str);
			free((void*)Str);
			return StandardStr;
		}
		return std::string();
	}

	IDeckLinkIterator* CreateDeckLinkIterator()
	{
		IDeckLinkIterator* DeckLinkIterator = CreateDeckLinkIteratorInstance();

		if (!DeckLinkIterator)
		{
			UE_LOGF(LogBlackmagicCore, Error,"A DeckLink iterator could not be created. The DeckLink drivers may not be installed.");
		}

		return DeckLinkIterator;
	}

	void DestroyDeckLinkIterator(IDeckLinkIterator* DeckLink)
	{
		if (DeckLink)
		{
			DeckLink->Release();
		}
	}

	IDeckLinkVideoConversion* CreateDeckLinkVideoConversion()
	{
		IDeckLinkVideoConversion* DeckLinkVideoConversion = CreateVideoConversionInstance();

		if (!DeckLinkVideoConversion)
		{
			UE_LOGF(LogBlackmagicCore, Error,"A DeckLink video conversion could not be created. The DeckLink drivers may not be installed.");
		}

		return DeckLinkVideoConversion;
	}

	void DestroyDeckLinkVideoConversion(IDeckLinkVideoConversion* DeckLink)
	{
		if (DeckLink)
		{
			DeckLink->Release();
		}
	}

	void SetThreadPriority_TimeCritical(std::thread& InThread)
	{
        pthread_attr_t Attributes;
        int Policy = 0;
        int Priority = 0;
        pthread_attr_init(&Attributes);
        pthread_attr_getschedpolicy(&Attributes, &Policy);
        Policy = sched_get_priority_max(Priority);

        pthread_setschedprio(InThread.native_handle(), Priority);

        pthread_attr_destroy(&Attributes);
	}

	bool GetDisplayName(IDeckLink* Device, TCHAR* OutDisplayName, int32_t Size)
	{
		const char* DisplayName = 0;
		HRESULT Result = Device->GetDisplayName(&DisplayName);
		if (Result == S_OK)
		{
			TStringConvert<UTF8CHAR, TCHAR> StringConvert;
			int32 SourceLength = FCStringAnsi::Strlen(DisplayName) + 1;
			int32 ConvertedLength = StringConvert.ConvertedLength(DisplayName, SourceLength);
			check(ConvertedLength < Size);
			StringConvert.Convert(OutDisplayName, ConvertedLength, DisplayName, SourceLength);
			return true;
		}

		return false;
	}

	void* Allocate(uint32_t Size)
	{
		// TODO: Use mmap with, would MAP_SHARED be needed?
		return malloc(Size);
	}

	bool Free(void* Address, uint32_t Size)
	{
		free(Address);
		return true;
	}
}

#endif