// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_LINUX

#include <codecvt>
#include <locale>
#include <thread>

#include "LinuxCOM.h"

// convert Windows style enums in Linux, i.e. BMDPixelFormat to _BMDPixelFormat
#define ENUM(x) _ ## x

#define BSTR const char*

#define LONGLONG int64_t

// simulate Windows BOOL type
typedef bool BOOL;

class IDeckLink;
class IDeckLinkDisplayMode;
class IDeckLinkIterator;
class IDeckLinkVideoConversion;

/** Defined outside of namespace to match windows' implementation */
bool IsEqualIID(REFIID LHS, REFIID RHS);

namespace BlackmagicPlatform
{
	std::string GetName(IDeckLinkDisplayMode* DisplayMode);

	bool InitializeAPI();
	void ReleaseAPI();

	IDeckLinkIterator* CreateDeckLinkIterator();
	void DestroyDeckLinkIterator(IDeckLinkIterator*);

	IDeckLinkVideoConversion* CreateDeckLinkVideoConversion();
	void DestroyDeckLinkVideoConversion(IDeckLinkVideoConversion*);

	void SetThreadPriority_TimeCritical(std::thread& InThread);

	bool GetDisplayName(IDeckLink* Device, TCHAR* OutDisplayName, int32_t Size);

	void* Allocate(uint32_t Size);
	bool Free(void* Address, uint32_t Size);
}

#endif