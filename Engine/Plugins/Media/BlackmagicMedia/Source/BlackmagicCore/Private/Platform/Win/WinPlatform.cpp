// Copyright Epic Games, Inc. All Rights Reserved.

#include "WinPlatform.h"

#if PLATFORM_WINDOWS

#include "Common.h"

#include <conio.h>
#include <objbase.h>

namespace BlackmagicPlatform
{

	bool InitializeAPI()
	{
		bool bResult = FWindowsPlatformMisc::CoInitialize();
		if (!bResult)
		{
			UE_LOGF(LogBlackmagicCore, Error,"Initialization of Blackmagic COM failed");
		}
		return bResult;
	}

	void ReleaseAPI()
	{
		FWindowsPlatformMisc::CoUninitialize();
	}

	std::string GetName(IDeckLinkDisplayMode* InDisplayMode)
	{
		if (InDisplayMode)
		{
			BSTR str;
			std::string stdStr;

			if (SUCCEEDED(InDisplayMode->GetName(&str)))
			{
				_bstr_t myBStr(str);
				stdStr = myBStr;
				SysFreeString(str);
			}

			return stdStr;
		}
		return std::string();
	}

	IDeckLinkIterator* CreateDeckLinkIterator()
	{
		IDeckLinkIterator* DeckLinkIterator = nullptr;
		HRESULT Result = CoCreateInstance(CLSID_CDeckLinkIterator, NULL, CLSCTX_ALL, IID_IDeckLinkIterator, (void**)&DeckLinkIterator);

		if (FAILED(Result))
		{
			UE_LOGF(LogBlackmagicCore, Error,"A DeckLink iterator could not be created - %08x. The DeckLink drivers may not be installed.", Result);
			DeckLinkIterator = nullptr;
		}

		return DeckLinkIterator;
	}

	void DestroyDeckLinkIterator(IDeckLinkIterator* DeckLink)
	{
		if (DeckLink)
		{
			DeckLink->Release();
		}
	}

	IDeckLinkVideoConversion* CreateDeckLinkVideoConversion()
	{
		IDeckLinkVideoConversion* DeckLinkVideoConversion = nullptr;
		HRESULT Result = CoCreateInstance(CLSID_CDeckLinkVideoConversion, NULL, CLSCTX_ALL, IID_IDeckLinkVideoConversion, (void**)&DeckLinkVideoConversion);

		if (FAILED(Result))
		{
			UE_LOGF(LogBlackmagicCore, Error,"A DeckLink video conversion could not be created - %08x. The DeckLink drivers may not be installed.", Result);
			DeckLinkVideoConversion = nullptr;
		}

		return DeckLinkVideoConversion;
	}

	void DestroyDeckLinkVideoConversion(IDeckLinkVideoConversion* DeckLink)
	{
		if (DeckLink)
		{
			DeckLink->Release();
		}
	}

	void SetThreadPriority_TimeCritical(std::thread& InThread)
	{
		SetThreadPriority(InThread.native_handle(), THREAD_PRIORITY_TIME_CRITICAL);
	}

	bool GetDisplayName(IDeckLink* Device, TCHAR* OutDisplayName, int32_t Size)
	{
		BSTR DeviceName = 0;
		HRESULT Result = Device->GetDisplayName(&DeviceName);
		if (Result == S_OK)
		{
			wcscpy_s(OutDisplayName, BlackmagicDesign::BlackmagicDeviceScanner::FormatedTextSize, DeviceName);
			::SysFreeString(DeviceName);
			return true;
		}

		return false;
	}

	void* Allocate(uint32_t BufferSize)
	{
		return VirtualAlloc(NULL, BufferSize, MEM_COMMIT | MEM_RESERVE | MEM_WRITE_WATCH, PAGE_READWRITE);
	}

	bool Free(void* Address, uint32_t BufferSize)
	{
		return VirtualFree(Address, BufferSize, MEM_RELEASE);
	}
}

#endif
