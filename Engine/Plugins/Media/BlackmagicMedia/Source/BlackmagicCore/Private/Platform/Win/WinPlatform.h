// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS

#include "CoreMinimal.h"

#include <thread>
#include <string>

// convert Windows style enums in Linux, i.e. BMDPixelFormat to _BMDPixelFormat
#define ENUM(x) x

#ifndef FALSE
#define FALSE 0
#endif

#ifndef TRUE
#define TRUE 1
#endif

struct IDeckLinkDisplayMode;
struct IDeckLinkIterator;
struct IDeckLinkVideoConversion;
struct IDeckLink;

namespace BlackmagicPlatform
{
	bool InitializeAPI();
	void ReleaseAPI();

	std::string GetName(IDeckLinkDisplayMode* DisplayMode);

	IDeckLinkIterator* CreateDeckLinkIterator();
	void DestroyDeckLinkIterator(IDeckLinkIterator*);

	IDeckLinkVideoConversion* CreateDeckLinkVideoConversion();
	void DestroyDeckLinkVideoConversion(IDeckLinkVideoConversion*);

	void SetThreadPriority_TimeCritical(std::thread& InThread);

	bool GetDisplayName(IDeckLink* Device, TCHAR* OutDisplayName, int32_t Size); 

	void* Allocate(uint32_t BufferSize);
	bool Free(void* Address, uint32_t BufferSize);
}

#endif // PLATFORM_WINDOWS