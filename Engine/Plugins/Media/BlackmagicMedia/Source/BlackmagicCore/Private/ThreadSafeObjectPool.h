// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <mutex>
#include <vector>

namespace Blackmagic
{
namespace Private
{
	class IPoolable
	{
	public:
		virtual ~IPoolable() {}
		/** Clear the object before it is reused by the pool. */
		virtual void Clear() = 0;
	};

	template <typename  ObjectType>
	class ThreadSafeObjectPool
	{
	public:
		ThreadSafeObjectPool(uint32_t PoolCapacity)
		{
			static_assert(std::is_convertible<ObjectType*, IPoolable*>::value, "ObjectType must inherit from IPoolable");
			Reserve(PoolCapacity);
		}

		/** Destructor. */
		~ThreadSafeObjectPool()
		{
			Reserve(0);
		}

	public:

		/** Acquire an object from the pool. */
		ObjectType* Acquire()
		{
			ObjectType* Result = nullptr;
			const std::lock_guard<std::mutex> ScopeLock(PoolMutex);
			if (Pool.size() > 0)
			{
				Result = Pool.back();
				Pool.pop_back();
			}
			
			return Result;
		}

		/** Get the number of objects stored. */
		size_t Num() const
		{
			const std::lock_guard<std::mutex> ScopeLock(PoolMutex);
			return Pool.size();
		}

		/** Return the given object to the pool. */
		void Release(ObjectType* Object)
		{
			if (Object == nullptr)
			{
				return;
			}

			const std::lock_guard<std::mutex> ScopeLock(PoolMutex);

			Object->Clear();
			Pool.push_back(Object);
		}

		/** Reserve the specified number of objects. */
		void Reserve(uint32_t NumObjects)
		{
			const std::lock_guard<std::mutex> ScopeLock(PoolMutex);

			while (NumObjects < (uint32_t)Pool.size())
			{
				delete Pool.back();
				Pool.pop_back();
			}

			while (NumObjects > (uint32_t)Pool.size())
			{
				Pool.push_back(new ObjectType());
			}
		}

	private:
		mutable std::mutex PoolMutex;

		/** List of unused objects. */
		std::vector<ObjectType*> Pool; 
	};
}
}
