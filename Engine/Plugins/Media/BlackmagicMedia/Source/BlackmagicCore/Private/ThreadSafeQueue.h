// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <algorithm>
#include <vector>
#include "Common.h"

namespace Blackmagic
{
	namespace Private
	{
		template<class T>
		class ThreadSafeQueue
		{
		public:
			void Push(const T& InValue)
			{
				const std::lock_guard<std::mutex> ScopeLock(QueueMutex);
				List.push_back(InValue);
			}

			bool Pop(T& OutValue)
			{
				const std::lock_guard<std::mutex> ScopeLock(QueueMutex);
				if (!List.empty())
				{
					OutValue = List.front();
					List.erase(List.begin());
					return true;
				}
				return false;
			}

			void Erase(const T& InValue)
			{
				const std::lock_guard<std::mutex> ScopeLock(QueueMutex);
				auto Found = std::find(List.begin(), List.end(), InValue);
				if (Found != List.end())
				{
					List.erase(Found);
				}
			}

			size_t Size() const
			{
				const std::lock_guard<std::mutex> ScopeLock(QueueMutex);
				return List.size();
			}

		private:
			
			mutable std::mutex QueueMutex;
			std::vector<T> List;
		};
	}
}
