// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/App.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBlackmagicCore, Log, All);

class FBlackmagicCoreModule : public IModuleInterface
{
public:

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**  Whether the API is initialized. */
	bool IsInitialized()
	{
		return bInitialized;
	}

	/** Whether a blackmagic card can be used. */
	bool CanUseBlackmagicCard() 
	{
		return FApp::CanEverRender() || bCanForceBlackmagicUsage;
	}

private:
	bool bCanForceBlackmagicUsage = false;
	bool bInitialized = false;
};

