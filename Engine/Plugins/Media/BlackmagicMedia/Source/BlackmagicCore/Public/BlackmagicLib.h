// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlackmagicCoreModule.h"
#include <cstdint>

#include "BlackmagicReferencePtr.h"

class FRHITexture;

namespace UE::GPUTextureTransfer
{
	class ITextureTransfer;
}

namespace BlackmagicDesign
{
	using LoggingCallbackPtr = void(*)(const TCHAR* Format, ...);
	using FBlackmagicVideoFormat = int32_t; //BMDDisplayMode

	enum class EPixelFormat
	{
		pf_8Bits,
		pf_10Bits
	};

	enum class EFullPixelFormat
	{
		pf_8BitBGRA,
		pf_8BitYUV,
		pf_10BitRGB,
		pf_10BitRGBXLE,
		pf_10BitYUV,
		pf_10BitRGBX,
	};

	enum class EPixelColor
	{
		YCbCr,
		RGB,
	};

	enum class EFieldDominance
	{
		Progressive,
		Interlaced,
		ProgressiveSegmentedFrame,
	};

	enum class EHDRMetaDataColorspace
	{
		Rec601,
		Rec709,
		Rec2020,
	};
	enum class EHDRMetaDataEOTF
	{
		SDR,
		HDR,
		PQ,
		HLG,
	};

	enum class EAudioBitDepth
	{
		Signed_16Bits,
		Signed_32Bits
	};

	enum class EAudioChannelConfiguration : uint8_t
	{
		Channels_2 = 2,
		Channels_8 = 8,
		Channels_16 = 16
	};

	enum class EAudioSampleRate : uint32_t
	{
		SR_48kHz = 48000
	};

	enum class ERHI : uint8_t
	{
		Invalid,
		D3D11,
		D3D12,
		Cuda,
		Vulkan
	};

	namespace Private
	{
		class DeviceScanner;
		class VideoFormatsScanner;
	}

	/* FUniqueIdentifier definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API FUniqueIdentifier
	{
		FUniqueIdentifier();
		explicit FUniqueIdentifier(int32_t InIdentifier);
		bool IsValid() const;
		bool operator== (const FUniqueIdentifier& rhs) const;

	private:
		int32_t Identifier;
	};

	/* FTimecode definition
	 * limited to 30fps
	*****************************************************************************/
	struct BLACKMAGICCORE_API FTimecode
	{
		FTimecode();
		bool operator== (const FTimecode& Other) const;

		uint32_t Hours;
		uint32_t Minutes;
		uint32_t Seconds;
		uint32_t Frames;
		bool bIsDropFrame;
	};

	enum struct ETimecodeFormat
	{
		TCF_None,
		TCF_LTC,
		TCF_VITC1,
		TCF_Auto
	};

	enum struct ELinkConfiguration
	{
		SingleLink,
		DualLink,
		QuadLinkTSI,
		QuadLinkSqr,
	};
 
	/* FFormatInfo definition
	 * Information about a given frame desc
	*****************************************************************************/
	struct BLACKMAGICCORE_API FFormatInfo
	{
		/** Framerate */
		uint32_t FrameRateNumerator = 0;
		uint32_t FrameRateDenominator = 0;

		/** Image Width & Weight in texels */
		uint32_t Width = 0;
		uint32_t Height = 0;

		EFieldDominance FieldDominance = EFieldDominance::Progressive;
		FBlackmagicVideoFormat DisplayMode = 0; // Unique identifier that represent all that combination for the device

		bool operator==(FFormatInfo& Other) const;

	};
	
	/* FAudioFormat definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API FAudioFormat
	{
		EAudioBitDepth BitDepth;
		EAudioChannelConfiguration ChannelConfigration;
		bool operator==(FAudioFormat& Other) const;
	};

	/* FChannelInfo definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API FChannelInfo
	{
		int32_t DeviceIndex;

		bool operator==(FChannelInfo& Other) const;
	};


	/** HDR Metadata. */
	struct BLACKMAGICCORE_API FHDRMetaData
	{
		FHDRMetaData();

		/** Whether the data contained in this struct is valid. */
		bool bIsAvailable;
		/** Target color space. */
		EHDRMetaDataColorspace ColorSpace;
		/** HDR Transfer function. */
		EHDRMetaDataEOTF EOTF;
		/** White point X coordinate. */
		double WhitePointX;
		/** White point Y coordinate. */
		double WhitePointY;
		/** Red chromaticity X coordinate. */
		double DisplayPrimariesRedX;
		/** Red chromaticity Y coordinate. */
		double DisplayPrimariesRedY;
		/** Green chromaticity X coordinate. */
		double DisplayPrimariesGreenX;
		/** Green chromaticity Y coordinate. */
		double DisplayPrimariesGreenY;
		/** Blue chromaticity X coordinate. */
		double DisplayPrimariesBlueX;
		/** Blue chromaticity Y coordinate. */
		double DisplayPrimariesBlueY;
		/** Max display mastering luminance. */
		double MaxDisplayLuminance;
		/** Min display mastering luminance. */
		double MinDisplayLuminance;
		/** Max content light level. */
		double MaxContentLightLevel;
		/** Max frame average light level. */
		double MaxFrameAverageLightLevel;
	};

	/* FInputChannelOptions definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API FInputChannelOptions
	{
		FInputChannelOptions();

		FFormatInfo FormatInfo;
		int32_t CallbackPriority;

		bool bReadVideo;
		EPixelFormat PixelFormat;
		
		ETimecodeFormat TimecodeFormat;

		bool bReadAudio;
		int32_t NumberOfAudioChannel;

		bool bUseTheDedicatedLTCInput;
		bool bAutoDetect;
	};

	/* FOutputChannelOptions definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API FOutputChannelOptions
	{
		FOutputChannelOptions();

		bool IsInterlaced() const
		{
			return FormatInfo.FieldDominance == EFieldDominance::Interlaced;
		}

		FFormatInfo FormatInfo;
		int32_t CallbackPriority;
		EPixelFormat PixelFormat;

		EAudioSampleRate AudioSampleRate;
		EAudioBitDepth AudioBitDepth;
		EAudioChannelConfiguration NumAudioChannels;

		uint32_t NumberOfBuffers;

		ETimecodeFormat TimecodeFormat;
		ELinkConfiguration LinkConfiguration;

		/** HDR Metadata. */
		FHDRMetaData HDRMetadata;

		bool bOutputKey;
		bool bOutputVideo;
		bool bOutputAudio;
		bool bOutputInterlacedFieldsTimecodeNeedToMatch;
		bool bOutputInterlaceAsProgressive;
		bool bOutputInterlaceOnEvenFrames;
		bool bLogDropFrames;
		bool bUseGPUDMA;
		bool bScheduleInDifferentThread;
		bool bEncodeTimecodeInTexel;
	};

	/* IInputEventCallback definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API IInputEventCallback
	{
		struct BLACKMAGICCORE_API FFrameReceivedInfo
		{
			FFrameReceivedInfo();

			bool bHasInputSource;

			int64_t FrameNumber;

			// Timecode
			bool bHaveTimecode;
			FTimecode Timecode;

			// Video
			void* VideoBuffer;
			int32_t VideoWidth;
			int32_t VideoHeight;
			int32_t VideoPitch;
			EPixelFormat PixelFormat;
			EFullPixelFormat FullPixelFormat;
			EFieldDominance FieldDominance;
			FHDRMetaData HDRMetaData;

			// Audio
			void* AudioBuffer;
			int32_t AudioBufferSize;
			int32_t NumberOfAudioChannel;
			int32_t AudioRate;
		};

		/** An empty type to expose. The real job is done in an internal child implementation. */
		struct BLACKMAGICCORE_API FFrameBufferHolder
		{
			virtual ~FFrameBufferHolder() = default;
		};

		/**
		 * A container for video and audio buffer holders. These holders prevent
		 * internal buffers from being released while they are in use.
		 */
		struct BLACKMAGICCORE_API FFrameReceivedBufferHolders
		{
			TSharedPtr<FFrameBufferHolder>* VideoBufferHolder = nullptr;
			TSharedPtr<FFrameBufferHolder>* AudioBufferHolder = nullptr;
		};

		virtual ~IInputEventCallback();

		virtual void AddRef() = 0;
		virtual void Release() = 0;

		virtual void OnInitializationCompleted(bool bSuccess) = 0;
		virtual void OnShutdownCompleted() = 0;

		virtual void OnFrameReceived(const FFrameReceivedInfo&)
		{ }

		virtual void OnFrameReceived(const FFrameReceivedInfo&, FFrameReceivedBufferHolders& /* out */)
		{ }

		virtual void OnFrameFormatChanged(const FFormatInfo& NewFormat) = 0;
		virtual void OnInterlacedOddFieldEvent(long long FrameNumber) = 0;
	};

	/* IOutputEventCallback definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API IOutputEventCallback
	{
		struct BLACKMAGICCORE_API FFrameSentInfo
		{
			FFrameSentInfo();

			uint32_t FramesLost;
			uint32_t FramesDropped;
		};

		virtual ~IOutputEventCallback();

		virtual void AddRef() = 0;
		virtual void Release() = 0;

		virtual void OnInitializationCompleted(bool bSuccess) = 0;
		virtual void OnShutdownCompleted() = 0;

		virtual void OnOutputFrameCopied(const FFrameSentInfo& InFrameInfo) = 0;
		virtual void OnPlaybackStopped() = 0;
		virtual void OnInterlacedOddFieldEvent() = 0;
	};

	struct BLACKMAGICCORE_API FBaseFrameData
	{
		FTimecode Timecode;
		uint32_t FrameIdentifier = 0;
		bool bEvenFrame = true;
	};

	struct BLACKMAGICCORE_API FFrameDescriptor : public FBaseFrameData
	{
		uint8_t* VideoBuffer = nullptr;
		int32_t VideoWidth = 0;
		int32_t VideoHeight = 0;
	};

	struct BLACKMAGICCORE_API FFrameDescriptor_GPUDMA : public FBaseFrameData
	{
		FRHITexture* RHITexture = nullptr;
	};

	struct BLACKMAGICCORE_API FAudioSamplesDescriptor
	{
		uint8_t* AudioBuffer = nullptr;
		int32_t AudioBufferLength = 0;
		int32_t NumAudioSamples = 0;

		FTimecode Timecode;
		uint32_t FrameIdentifier = 0;
	};

	/* BlackmagicDeviceScanner definition
	*****************************************************************************/
	class BLACKMAGICCORE_API BlackmagicDeviceScanner
	{
	public:
		const static int32_t FormatedTextSize = 64;
		using FormatedTextType = TCHAR[FormatedTextSize];

		struct BLACKMAGICCORE_API DeviceInfo
		{
			bool bIsSupported;
			bool bCanDoCapture;
			bool bCanDoPlayback;
			bool bCanDoFullDuplex;
			bool bCanDoDualLink;
			bool bCanDoQuadLink;
			bool bCanDoQuadSquareLink;
			bool bHasGenlockReferenceInput;
			bool bHasLTCTimecodeInput;
			bool bCanAutoDetectInputFormat;
			bool bSupportInternalKeying;
			bool bSupportExternalKeying;

			uint32_t NumberOfSubDevices;
			uint32_t DevicePersistentId;
			uint32_t ProfileId;
			uint32_t DeviceGroupId;
			uint32_t SubDeviceIndex;
		};

		BlackmagicDeviceScanner();
		~BlackmagicDeviceScanner();

		BlackmagicDeviceScanner(const BlackmagicDeviceScanner&) = delete;
		BlackmagicDeviceScanner& operator=(const BlackmagicDeviceScanner&) = delete;

		int32_t GetNumDevices() const;
		bool GetDeviceTextId(int32_t InDeviceIndex, FormatedTextType& OutTextId) const;
		bool GetDeviceInfo(int32_t InDeviceIndex, DeviceInfo& OutDeviceInfo) const;

	private:
		Private::DeviceScanner* Scanner;
	};

	/* BlackmagicVideoFormats definition
	*****************************************************************************/
	struct BLACKMAGICCORE_API BlackmagicVideoFormats
	{
		struct BLACKMAGICCORE_API VideoFormatDescriptor
		{
			VideoFormatDescriptor();

			FBlackmagicVideoFormat VideoFormatIndex;
			uint32_t FrameRateNumerator;
			uint32_t FrameRateDenominator;
			uint32_t ResolutionWidth;
			uint32_t ResolutionHeight;
			bool bIsProgressiveStandard;
			bool bIsInterlacedStandard;
			bool bIsPsfStandard;
			bool bIsSD;
			bool bIsHD;
			bool bIs2K;
			bool bIs4K;
			bool bIs8K;

			bool bIsValid;
		};

		BlackmagicVideoFormats(int32_t InDeviceId, bool bForOutput);
		~BlackmagicVideoFormats();

		BlackmagicVideoFormats(const BlackmagicVideoFormats&) = delete;
		BlackmagicVideoFormats& operator=(const BlackmagicVideoFormats&) = delete;

		int32_t GetNumSupportedFormat() const;
		VideoFormatDescriptor GetSupportedFormat(int32_t InIndex) const;

	private:
		Private::VideoFormatsScanner* Formats;
	};

	 /* Configure Logging
	 *****************************************************************************/
	BLACKMAGICCORE_API void SetLoggingCallbacks(LoggingCallbackPtr LogInfoFunc, LoggingCallbackPtr LogWarningFunc, LoggingCallbackPtr LogErrorFunc, LoggingCallbackPtr LogVerboseFunc);

	 /* Initialization
	 *****************************************************************************/
	BLACKMAGICCORE_API bool ApiInitialization();
	BLACKMAGICCORE_API void ApiUninitialization();

	 /* Register/Unregister
	 *****************************************************************************/
	BLACKMAGICCORE_API FUniqueIdentifier RegisterCallbackForChannel(const FChannelInfo& InChannelInfo, const FInputChannelOptions& InChannelOptions, ReferencePtr<IInputEventCallback> InCallback);
	BLACKMAGICCORE_API void UnregisterCallbackForChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier);
	
	BLACKMAGICCORE_API FUniqueIdentifier RegisterOutputChannel(const FChannelInfo& InChannelInfo, const FOutputChannelOptions& InChannelOptions, ReferencePtr<IOutputEventCallback> InCallback);
	BLACKMAGICCORE_API void UnregisterOutputChannel(const FChannelInfo& InChannelInfo, FUniqueIdentifier InIdentifier, bool bCallCompleted);
	BLACKMAGICCORE_API bool SendVideoFrameData(const FChannelInfo& InChannelInfo, const FFrameDescriptor& InFrame);
	BLACKMAGICCORE_API bool SendVideoFrameData(const FChannelInfo& InChannelInfo, FFrameDescriptor_GPUDMA& InFrame);
	// Return whether to capture or discard a frame based on its timecode and frame number.
	BLACKMAGICCORE_API bool ShouldCaptureThisFrame(const FChannelInfo& InChannelInfo, ::FTimecode Timecode, uint32 FrameNumber);
	BLACKMAGICCORE_API bool SendAudioSamples(const FChannelInfo& InChannelInfo, const FAudioSamplesDescriptor& InSamples);
};
