// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstddef>

namespace BlackmagicDesign
{
	template<class T>
	class ReferencePtr
	{
	public:
		ReferencePtr()
			: Pointer(nullptr)
		{ }

		explicit ReferencePtr(T* InPointer)
			: Pointer(InPointer)
		{
			if (Pointer)
			{
				Pointer->AddRef();
			}
		}

		ReferencePtr(const ReferencePtr<T>& InAutoPointer)
			: Pointer(InAutoPointer.Pointer)
		{
			if (Pointer)
			{
				Pointer->AddRef();
			}
		}

		ReferencePtr(ReferencePtr&& InAutoPointer)
			: Pointer(InAutoPointer.Pointer)
		{
			InAutoPointer.Pointer = nullptr;
		}

		ReferencePtr& operator=(const ReferencePtr& InAutoPointer)
		{
			Reset();
			Pointer = InAutoPointer.Pointer;
			if (Pointer)
			{
				Pointer->AddRef();
			}
			return *this;
		}

		ReferencePtr& operator=(ReferencePtr&& InAutoPointer)
		{
			Reset();
			Pointer = InAutoPointer.Pointer;
			InAutoPointer.Pointer = nullptr;
			return *this;
		}

		~ReferencePtr()
		{
			if (Pointer)
			{
				Pointer->Release();
			}
		}

		T* operator->()
		{
			return Pointer;
		}

		void Reset()
		{
			if (Pointer)
			{
				Pointer->Release();
				Pointer = nullptr;
			}
		}

		T* Get()
		{
			return Pointer;
		}

		const T* Get() const
		{
			return Pointer;
		}

		operator T*()
		{
			return Pointer;
		}

		operator const T*() const
		{
			return Pointer;
		}

	private:
		T* Pointer;
	};

	template<class T >
	bool operator==(const ReferencePtr<T>& lhs, std::nullptr_t rhs) noexcept { return lhs.Get() == nullptr; }
	template< class T >
	bool operator==(std::nullptr_t lhs, const ReferencePtr<T>& rhs) noexcept { return rhs.Get() == nullptr; }
	template< class T >
	bool operator!=(const ReferencePtr<T>& lhs, std::nullptr_t rhs) noexcept { return lhs.Get() != nullptr; }
	template< class T >
	bool operator!=(std::nullptr_t lhs, const ReferencePtr<T>& rhs) noexcept { return rhs.Get() != nullptr; }
};
