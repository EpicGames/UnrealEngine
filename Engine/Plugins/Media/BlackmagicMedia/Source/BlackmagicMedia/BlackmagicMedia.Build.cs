// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	using System.IO;

	public class BlackmagicMedia : ModuleRules
	{
		public BlackmagicMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
			IWYUSupport = IWYUSupport.None;

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"MediaAssets",
					"MediaIOCore",
					"RHI",
					"TimeManagement",
					"RenderCore"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"BlackmagicCore",
					"Engine",
					"MediaUtils",
					"OpenColorIO",
					"Projects",
					"SlateCore"
                });

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});
		}
	}
}
