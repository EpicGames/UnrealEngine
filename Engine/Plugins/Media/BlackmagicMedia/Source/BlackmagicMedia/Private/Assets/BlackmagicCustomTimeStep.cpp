// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicCustomTimeStep.h"

#include "BlackmagicMediaPrivate.h"
#include "IBlackmagicMediaModule.h"

#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"

#include "Misc/App.h"
#include "Templates/Atomic.h"


namespace BlackmagicCustomTimeStepHelpers
{
	//~ FInputEventCallback implementation
	//--------------------------------------------------------------------
	class FInputEventCallback : public BlackmagicDesign::IInputEventCallback
	{
	public:
		FInputEventCallback(UBlackmagicCustomTimeStep* InCustomTimeStep, const BlackmagicDesign::FChannelInfo& InChannelInfo, bool bInEnableOverrunDetection, uint32 InExpectedSyncDelta, bool bInWaitForBothFields)
			: WeakOwner(InCustomTimeStep)
			, RefCounter(0)
			, ChannelInfo(InChannelInfo)
			, State(ECustomTimeStepSynchronizationState::Closed)
			, WaitSyncEvent(nullptr)
			, bWaitedOnce(false)
			, bEnableOverrunDetection(bInEnableOverrunDetection)
			, CurrentSyncCount(0)
			, bIsPreviousSyncCountValid(false)
			, PreviousSyncCount(0)
			, SyncCountDelta(1)
			, ExpectedSyncCountDelta(InExpectedSyncDelta)
			, bWaitForBothFields(bInWaitForBothFields)
		{
		}

		virtual ~FInputEventCallback()
		{
			if (WaitSyncEvent)
			{
				FPlatformProcess::ReturnSynchEventToPool(WaitSyncEvent);
			}
		}


		bool Initialize(const BlackmagicDesign::FInputChannelOptions& InChannelOptions)
		{
			AddRef();

			BlackmagicDesign::ReferencePtr<BlackmagicDesign::IInputEventCallback> SelfRef(this);
			BlackmagicIdendifier = BlackmagicDesign::RegisterCallbackForChannel(ChannelInfo, InChannelOptions, SelfRef);
			State = BlackmagicIdendifier.IsValid() ? ECustomTimeStepSynchronizationState::Synchronizing : ECustomTimeStepSynchronizationState::Error;
			return BlackmagicIdendifier.IsValid();
		}

		void Uninitialize()
		{
			if (BlackmagicIdendifier.IsValid())
			{
				BlackmagicDesign::UnregisterCallbackForChannel(ChannelInfo, BlackmagicIdendifier);
				BlackmagicIdendifier = BlackmagicDesign::FUniqueIdentifier();
			}

			Release();
		}

		ECustomTimeStepSynchronizationState GetSynchronizationState() const 
		{ 
			return State; 
		}

		bool WaitForSync()
		{
			bool bWaitIsValid = false;

			SyncCountDelta = 1;

			if (!WaitSyncEvent || (State != ECustomTimeStepSynchronizationState::Synchronized))
			{
				bIsPreviousSyncCountValid = false;
				return false;
			}

			constexpr uint32 MaxWaitForSyncInMs = 100;

			if (!WaitSyncEvent->Wait(MaxWaitForSyncInMs))
			{
				bIsPreviousSyncCountValid = false;
				State = ECustomTimeStepSynchronizationState::Error;
				return false;
			}

			uint64 NewSyncCount = CurrentSyncCount;

			if (bIsPreviousSyncCountValid)
			{
				SyncCountDelta = NewSyncCount - PreviousSyncCount;
			}

			PreviousSyncCount = NewSyncCount;

			if (bEnableOverrunDetection && bIsPreviousSyncCountValid && SyncCountDelta != ExpectedSyncCountDelta)
			{
				UE_LOGF(LogBlackmagicMedia, Warning, "The Engine couldn't run fast enough to keep up with the CustomTimeStep Sync. '%lld' frame(s) was dropped.", SyncCountDelta - ExpectedSyncCountDelta);
			}

			bIsPreviousSyncCountValid = true;

			return true;
		}

		uint64 GetLastSyncCountDelta() const
		{
			return SyncCountDelta;
		}

		bool IsLastSyncDataValid() const
		{
			return bIsPreviousSyncCountValid;
		}

	private:

		virtual void AddRef() override
		{
			++RefCounter;
		}

		virtual void Release() override
		{
			--RefCounter;
			if (RefCounter == 0)
			{
				delete this;
			}
		}
		virtual void OnInitializationCompleted(bool bSuccess) override
		{
			if (bSuccess)
			{
				const bool bIsManualReset = false;
				WaitSyncEvent = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
			}
			else
			{
				State = ECustomTimeStepSynchronizationState::Error;
			}
		}

		virtual void OnShutdownCompleted() override
		{
			State = ECustomTimeStepSynchronizationState::Closed;
			if (WaitSyncEvent)
			{
				WaitSyncEvent->Trigger();
			}
		}

		virtual void OnFrameReceived(const BlackmagicDesign::IInputEventCallback::FFrameReceivedInfo& InFrameInfo) override
		{
			if (!bWaitedOnce)
			{
				bWaitedOnce = true;
				State = ECustomTimeStepSynchronizationState::Synchronized;
			}

			CurrentSyncCount = InFrameInfo.FrameNumber;

			if (WaitSyncEvent)
			{
				WaitSyncEvent->Trigger();
			}
		}

		virtual void OnFrameFormatChanged(const BlackmagicDesign::FFormatInfo& NewFormat) override
		{
			UBlackmagicCustomTimeStep* TimeStep = WeakOwner.Get();
			if (TimeStep && TimeStep->bAutoDetectFormat)
			{
				TimeStep->MediaConfiguration.MediaMode.Resolution.X = NewFormat.Width;
				TimeStep->MediaConfiguration.MediaMode.Resolution.Y = NewFormat.Height;
				switch (NewFormat.FieldDominance)
				{
				case BlackmagicDesign::EFieldDominance::Interlaced:
					TimeStep->MediaConfiguration.MediaMode.Standard = EMediaIOStandardType::Interlaced;
					break;
				case BlackmagicDesign::EFieldDominance::Progressive:
					TimeStep->MediaConfiguration.MediaMode.Standard = EMediaIOStandardType::Progressive;
					break;
				case BlackmagicDesign::EFieldDominance::ProgressiveSegmentedFrame:
					TimeStep->MediaConfiguration.MediaMode.Standard = EMediaIOStandardType::Progressive;
					break;
				default:
					break;
				}

				TimeStep->MediaConfiguration.MediaMode.FrameRate = FFrameRate(NewFormat.FrameRateNumerator, NewFormat.FrameRateDenominator);
			}
			else
			{
				UE_LOGF(LogBlackmagicMedia, Error, "The video format changed.");
				State = ECustomTimeStepSynchronizationState::Error;
			}

			if (WaitSyncEvent)
			{
				WaitSyncEvent->Trigger();
			}
		}

		virtual void OnInterlacedOddFieldEvent(int64 FrameNumber) override
		{
			CurrentSyncCount = FrameNumber;

			if (WaitSyncEvent && !bWaitForBothFields)
			{
				WaitSyncEvent->Trigger();
			}
		}

	private:
		TWeakObjectPtr<UBlackmagicCustomTimeStep> WeakOwner;

		TAtomic<int32> RefCounter;

		BlackmagicDesign::FUniqueIdentifier BlackmagicIdendifier;
		BlackmagicDesign::FChannelInfo ChannelInfo;

		ECustomTimeStepSynchronizationState State;

		FEvent* WaitSyncEvent;
		bool bWaitedOnce;

		bool bEnableOverrunDetection;

		TAtomic<uint64> CurrentSyncCount;

		/** Remember the last Sync Count*/
		bool bIsPreviousSyncCountValid;
		uint64 PreviousSyncCount;
		uint64 SyncCountDelta;
		uint64 ExpectedSyncCountDelta;

		/** Whether to wait for both fields before triggering the WaitForSync event. */
		bool bWaitForBothFields = false;
	};
}

UBlackmagicCustomTimeStep::UBlackmagicCustomTimeStep(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableOverrunDetection(false)
	, InputEventCallback(nullptr)
	, bWarnedAboutVSync(false)
{
	MediaConfiguration.bIsInput = true;
}

bool UBlackmagicCustomTimeStep::Initialize(class UEngine* InEngine)
{
	if (!MediaConfiguration.IsValid())
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The configuration of '%ls' is not valid.", *GetName());
		return false;
	}

	check(InputEventCallback == nullptr);
	
	if (!IBlackmagicMediaModule::Get().IsInitialized())
	{
		UE_LOGF(LogBlackmagicMedia, Error, "The CustomTimeStep '%ls' can't be initialized. Blackmagic is not initialized on your machine.", *GetName());
		return false;
	}

	BlackmagicDesign::FChannelInfo ChannelInfo;
	ChannelInfo.DeviceIndex = MediaConfiguration.MediaConnection.Device.DeviceIdentifier;

	InputEventCallback = new BlackmagicCustomTimeStepHelpers::FInputEventCallback(this, ChannelInfo, bEnableOverrunDetection, GetExpectedSyncCountDelta(), bWaitForBothFields);

	BlackmagicDesign::FInputChannelOptions ChannelOptions;
	ChannelOptions.bAutoDetect = bAutoDetectFormat;
	ChannelOptions.CallbackPriority = 1;
	ChannelOptions.FormatInfo.DisplayMode = MediaConfiguration.MediaMode.DeviceModeIdentifier;
	ChannelOptions.FormatInfo.FrameRateNumerator = MediaConfiguration.MediaMode.FrameRate.Numerator;
	ChannelOptions.FormatInfo.FrameRateDenominator = MediaConfiguration.MediaMode.FrameRate.Denominator;

	switch (MediaConfiguration.MediaMode.Standard)
	{
	case EMediaIOStandardType::Interlaced:
		ChannelOptions.FormatInfo.FieldDominance = BlackmagicDesign::EFieldDominance::Interlaced;
		break;
	case EMediaIOStandardType::ProgressiveSegmentedFrame:
		ChannelOptions.FormatInfo.FieldDominance = BlackmagicDesign::EFieldDominance::ProgressiveSegmentedFrame;
		break;
	case EMediaIOStandardType::Progressive:
	default:
		ChannelOptions.FormatInfo.FieldDominance = BlackmagicDesign::EFieldDominance::Progressive;
		break;
	}

	ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_None;

	bool bResult = InputEventCallback->Initialize(ChannelOptions);
	if (!bResult)
	{
		ReleaseResources();
	}

	return bResult;
}

void UBlackmagicCustomTimeStep::Shutdown(class UEngine* InEngine)
{
	ReleaseResources();
}

bool UBlackmagicCustomTimeStep::UpdateTimeStep(class UEngine* InEngine)
{
	bool bRunEngineTimeStep = true;

	const ECustomTimeStepSynchronizationState CurrentState = GetSynchronizationState();

	if (CurrentState == ECustomTimeStepSynchronizationState::Synchronized)
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
		if (!bWarnedAboutVSync)
		{
			bool bLockToVsync = CVar->GetValueOnGameThread() != 0;
			if (bLockToVsync)
			{
				UE_LOGF(LogBlackmagicMedia, Warning, "The Engine is using VSync and may break the 'genlock'");
				bWarnedAboutVSync = true;
			}
		}

		// Updates logical last time to match logical current time from last tick
		UpdateApplicationLastTime();

		const double TimeBeforeSync = FPlatformTime::Seconds();
		WaitForSync();
		const double TimeAfterSync = FPlatformTime::Seconds();

		UpdateAppTimes(TimeBeforeSync, TimeAfterSync);

		bRunEngineTimeStep = false;
	}
	else if (CurrentState == ECustomTimeStepSynchronizationState::Error)
	{
		ReleaseResources();
	}

	return bRunEngineTimeStep;
}

ECustomTimeStepSynchronizationState UBlackmagicCustomTimeStep::GetSynchronizationState() const
{
	if (InputEventCallback)
	{
		return InputEventCallback->GetSynchronizationState();
	}
	return ECustomTimeStepSynchronizationState::Closed;
}

FFrameRate UBlackmagicCustomTimeStep::GetFixedFrameRate() const
{
	return MediaConfiguration.MediaMode.FrameRate;
}

void UBlackmagicCustomTimeStep::BeginDestroy()
{
	ReleaseResources();
	Super::BeginDestroy();
}

bool UBlackmagicCustomTimeStep::WaitForSync()
{
	if (!InputEventCallback)
	{
		return false;
	}

	return InputEventCallback->WaitForSync();
}

void UBlackmagicCustomTimeStep::ReleaseResources()
{
	if (InputEventCallback)
	{
		InputEventCallback->Uninitialize();
		InputEventCallback = nullptr;
	}
}

FFrameRate UBlackmagicCustomTimeStep::GetSyncRate() const
{
	return GetFixedFrameRate();
}

uint32 UBlackmagicCustomTimeStep::GetLastSyncCountDelta() const
{
	if (!InputEventCallback)
	{
		return 1;
	}

	// deltas are not expected to be big (a delta bigger than uint32 implies years)
	return uint32(InputEventCallback->GetLastSyncCountDelta());
}

bool UBlackmagicCustomTimeStep::IsLastSyncDataValid() const
{
	if (!InputEventCallback)
	{
		return false;
	}

	return InputEventCallback->IsLastSyncDataValid();
}
