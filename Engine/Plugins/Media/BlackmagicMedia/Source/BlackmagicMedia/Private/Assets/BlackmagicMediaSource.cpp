// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicMediaSource.h"

#include "BlackmagicDeviceProvider.h"
#include "BlackmagicMediaPrivate.h"
#include "IBlackmagicMediaModule.h"
#include "IMediaIOCoreModule.h"

#include "MediaIOCorePlayerBase.h"

UBlackmagicMediaSource::UBlackmagicMediaSource()
	: UCaptureCardMediaSource()
	, bCaptureAudio(false)
	, AudioChannels(EBlackmagicMediaAudioChannel::Stereo2)
	, MaxNumAudioFrameBuffer(8)
	, bCaptureVideo(true)
	, ColorFormat(EBlackmagicMediaSourceColorFormat::YUV8)
	, MaxNumVideoFrameBuffer(8)
	, bLogDropFrame(false)
	, bEncodeTimecodeInTexel(false)
{
}

/*
 * IMediaOptions interface
 */

bool UBlackmagicMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == BlackmagicMediaOption::CaptureAudio) { return bCaptureAudio; }
	if (Key == BlackmagicMediaOption::CaptureVideo) { return bCaptureVideo; }
	if (Key == BlackmagicMediaOption::LogDropFrame) { return bLogDropFrame; }
	if (Key == BlackmagicMediaOption::EncodeTimecodeInTexel) { return bEncodeTimecodeInTexel; }

	return Super::GetMediaOption(Key, DefaultValue);
}

int64 UBlackmagicMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == FMediaIOCoreMediaOption::FrameRateNumerator) { return MediaConfiguration.MediaMode.FrameRate.Numerator; }
	if (Key == FMediaIOCoreMediaOption::FrameRateDenominator) { return MediaConfiguration.MediaMode.FrameRate.Denominator; }
	if (Key == FMediaIOCoreMediaOption::ResolutionWidth) { return MediaConfiguration.MediaMode.Resolution.X; }
	if (Key == FMediaIOCoreMediaOption::ResolutionHeight) { return MediaConfiguration.MediaMode.Resolution.Y; }

	if (Key == BlackmagicMediaOption::DeviceIndex) { return MediaConfiguration.MediaConnection.Device.DeviceIdentifier; }
	if (Key == BlackmagicMediaOption::TimecodeFormat) { return (int64)AutoDetectableTimecodeFormat; }
	if (Key == BlackmagicMediaOption::AudioChannelOption) { return (int64)AudioChannels; }
	if (Key == BlackmagicMediaOption::MaxAudioFrameBuffer) { return MaxNumAudioFrameBuffer; }
	if (Key == BlackmagicMediaOption::BlackmagicVideoFormat) { return MediaConfiguration.MediaMode.DeviceModeIdentifier; }
	if (Key == BlackmagicMediaOption::ColorFormat) { return (int64)ColorFormat; }
	if (Key == BlackmagicMediaOption::MaxVideoFrameBuffer) { return MaxNumVideoFrameBuffer; }

	return Super::GetMediaOption(Key, DefaultValue);
}

FString UBlackmagicMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	if (Key == FMediaIOCoreMediaOption::VideoModeName)
	{
		return MediaConfiguration.MediaMode.GetModeName().ToString();
	}
	return Super::GetMediaOption(Key, DefaultValue);
}

bool UBlackmagicMediaSource::HasMediaOption(const FName& Key) const
{
	if (   Key == BlackmagicMediaOption::CaptureAudio
		|| Key == BlackmagicMediaOption::CaptureVideo
		|| Key == BlackmagicMediaOption::LogDropFrame
		|| Key == BlackmagicMediaOption::EncodeTimecodeInTexel)
	{
		return true;
	}

	if (   Key == BlackmagicMediaOption::DeviceIndex
		|| Key == BlackmagicMediaOption::TimecodeFormat
		|| Key == BlackmagicMediaOption::AudioChannelOption
		|| Key == BlackmagicMediaOption::MaxAudioFrameBuffer
		|| Key == BlackmagicMediaOption::BlackmagicVideoFormat
		|| Key == BlackmagicMediaOption::ColorFormat
		|| Key == BlackmagicMediaOption::MaxVideoFrameBuffer)
	{
		return true;
	}

	if (   Key == FMediaIOCoreMediaOption::FrameRateNumerator
		|| Key == FMediaIOCoreMediaOption::FrameRateDenominator
		|| Key == FMediaIOCoreMediaOption::ResolutionWidth
		|| Key == FMediaIOCoreMediaOption::ResolutionHeight
		|| Key == FMediaIOCoreMediaOption::VideoModeName)
	{
		return true;
	}

	return Super::HasMediaOption(Key);
}

/*
 * UMediaSource interface
 */

FString UBlackmagicMediaSource::GetUrl() const
{
	return MediaConfiguration.MediaConnection.ToUrl();
}

bool UBlackmagicMediaSource::Validate() const
{
	FString FailureReason;
	if (!MediaConfiguration.IsValid())
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaConfiguration '%ls' is invalid.", *GetName());
		return false;
	}

	if (!IBlackmagicMediaModule::Get().IsInitialized())
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "Can't validate MediaSource '%ls'. the Blackmagic library was not initialized.", *GetName());
		return false;
	}
	
	TUniquePtr<BlackmagicDesign::BlackmagicDeviceScanner> Scanner = MakeUnique<BlackmagicDesign::BlackmagicDeviceScanner>();
	BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo DeviceInfo;
	if (!Scanner->GetDeviceInfo(MediaConfiguration.MediaConnection.Device.DeviceIdentifier, DeviceInfo))
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' use the device '%ls' that doesn't exist on this machine.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (!DeviceInfo.bIsSupported)
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' use the device '%ls' that is not supported by the Blackmagic SDK.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (!DeviceInfo.bCanDoCapture)
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' use the device '%ls' that can't capture.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (bUseTimeSynchronization && AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None)
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' use time synchronization but doesn't enabled the timecode.", *GetName());
		return false;
	}


	if (EvaluationType == EMediaIOSampleEvaluationType::Timecode && (!bUseTimeSynchronization || AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None))
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' uses 'Timecode' evaluation type which requires time synchronization and timecode enabled.", *GetName());
		return false;
	}

	if (bFramelock && (!bRenderJIT || !bUseTimeSynchronization || AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None))
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' uses 'Framelock' which requires JIT rendering, time synchronization and timecode enabled.", *GetName());
		return false;
	}

	if (!bRenderJIT && EvaluationType == EMediaIOSampleEvaluationType::Latest)
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' uses 'Latest' evaluation type which requires JIT rendering.", *GetName());
		return false;
	}

	if (bFramelock)
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The MediaSource '%ls' uses 'Framelock' which has not been implemented yet. This option will be ignored.", *GetName());
	}

	return true;
}

#if WITH_EDITOR
FString UBlackmagicMediaSource::GetDescriptionString() const
{
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FBlackmagicDeviceProvider::GetProviderName());
	if (DeviceProviderPtr)
	{
		return DeviceProviderPtr->ToText(MediaConfiguration, bAutoDetectInput).ToString(); 
	}

	return Super::GetDescriptionString();
}

void UBlackmagicMediaSource::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
#define LOCTEXT_NAMESPACE "BlackmagicMediaSource"
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FBlackmagicDeviceProvider::GetProviderName());
	if (!DeviceProviderPtr)
	{
		return;
	}
	
	const FText ConfigHeader = LOCTEXT("MediaConfigurationHeader", "Media Configuration");

	const FText SourceText = FText::Format(LOCTEXT("TransportFormat", "{0} {1}"),
		DeviceProviderPtr->GetTransportName(MediaConfiguration.MediaConnection.TransportType, MediaConfiguration.MediaConnection.QuadTransportType),
		FText::AsNumber(MediaConfiguration.MediaConnection.PortIdentifier));
	
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("DeviceLabel", "Device"), FText::FromName(MediaConfiguration.MediaConnection.Device.DeviceName)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("SourceLabel", "Source"), SourceText));

	// When auto-detect is on, prefer the live-detected mode published by a running player.
	const bool bShowAuto = bAutoDetectInput;
	const bool bDetectedConfigurationAvailable = bShowAuto && HasLastDetectedConfiguration();
	const FMediaIOConfiguration& EffectiveConfig = bDetectedConfigurationAvailable ? GetLastDetectedConfiguration() : MediaConfiguration;
	const FText AutoNoSignal = LOCTEXT("AutoNoSignal", "Auto (no signal)");

	const FText ResolutionText = (bShowAuto && !bDetectedConfigurationAvailable) ? AutoNoSignal : EffectiveConfig.MediaMode.GetModeName();
	const FText StandardText = (bShowAuto && !bDetectedConfigurationAvailable) ? AutoNoSignal : UEnum::GetDisplayValueAsText(EffectiveConfig.MediaMode.Standard);
	const FText FramerateText = (bShowAuto && !bDetectedConfigurationAvailable) ? AutoNoSignal : EffectiveConfig.MediaMode.FrameRate.ToPrettyText();

	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("ResolutionLabel", "Resolution"), ResolutionText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("StandardLabel", "Standard"), StandardText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("FramerateLabel", "Framerate"), FramerateText));

	const FText AdditionalHeader = LOCTEXT("AdditionalSettingsHeader", "Additional Settings");
	const FText DisabledText = LOCTEXT("Disabled", "Disabled");
	const FText AudioText = FText::Format(LOCTEXT("AudioFormat", "{0}, {1} buffered frames"),
		UEnum::GetDisplayValueAsText(AudioChannels), FText::AsNumber(MaxNumAudioFrameBuffer));
	
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("TimecodeFormatLabel", "Timecode Format"), UEnum::GetDisplayValueAsText(AutoDetectableTimecodeFormat)));
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("AudioLabel", "Audio"), bCaptureAudio ? AudioText : DisabledText));
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("ColorConversionLabel", "Color Conversion"), FText::FromString(ColorConversionSettings.ToString())));
	
#undef LOCTEXT_NAMESPACE
}
#endif //WITH_EDITOR

void UBlackmagicMediaSource::AssignDefaultConfiguration()
{
	const FBlackmagicDeviceProvider DeviceProvider;
	const TArray<FMediaIOConfiguration> Configurations = DeviceProvider.GetConfigurations();

	for (const FMediaIOConfiguration& Configuration : Configurations)
	{
		if (Configuration.bIsInput)
		{
			MediaConfiguration = Configuration;
			bRenderJIT = false;
			break;
		}
	}
}

#if WITH_EDITOR
bool UBlackmagicMediaSource::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackmagicMediaSource, bEncodeTimecodeInTexel))
	{
		return AutoDetectableTimecodeFormat != EMediaIOAutoDetectableTimecodeFormat::None && bCaptureVideo;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTimeSynchronizableMediaSource, bUseTimeSynchronization))
	{
		return AutoDetectableTimecodeFormat != EMediaIOAutoDetectableTimecodeFormat::None;
	}

	return true;
}

void UBlackmagicMediaSource::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackmagicMediaSource, AutoDetectableTimecodeFormat))
	{
		if (AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None)
		{
			bUseTimeSynchronization = false;
			bEncodeTimecodeInTexel = false;
			EvaluationType = EMediaIOSampleEvaluationType::PlatformTime;
			bFramelock = false;
		}
	}

	if (InPropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, EvaluationType))
	{
		// 'Timecode' evaluation is not allowed if no timecode set
		if (AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None)
		{
			EvaluationType = (EvaluationType == EMediaIOSampleEvaluationType::Timecode ? EMediaIOSampleEvaluationType::PlatformTime : EvaluationType);
		}

		// 'Latest' evaluation is available in JITR only
		if (!bRenderJIT)
		{
			EvaluationType = (EvaluationType == EMediaIOSampleEvaluationType::Latest ? EMediaIOSampleEvaluationType::PlatformTime : EvaluationType);
		}
	}

	Super::PostEditChangeChainProperty(InPropertyChangedEvent);
}
#endif //WITH_EDITOR

void UBlackmagicMediaSource::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (TimecodeFormat_DEPRECATED != EMediaIOTimecodeFormat::None)
		{
			switch (TimecodeFormat_DEPRECATED)
			{
			case EMediaIOTimecodeFormat::LTC:
				AutoDetectableTimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::LTC;
				break;
			case EMediaIOTimecodeFormat::VITC:
				AutoDetectableTimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::VITC;
				break;
			default:
				break;
			}
			TimecodeFormat_DEPRECATED = EMediaIOTimecodeFormat::None;
		}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}