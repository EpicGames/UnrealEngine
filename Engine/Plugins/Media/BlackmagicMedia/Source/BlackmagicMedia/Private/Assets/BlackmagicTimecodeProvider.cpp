// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicTimecodeProvider.h"
#include "BlackmagicDeviceProvider.h"
#include "BlackmagicMediaPrivate.h"
#include "IBlackmagicMediaModule.h"

#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Templates/Atomic.h"


namespace BlackmagicTimecodeProviderHelpers
{
	//~ FEventCallback implementation
	//--------------------------------------------------------------------
	class FEventCallback : public BlackmagicDesign::IInputEventCallback
	{
	public:
		FEventCallback(UBlackmagicTimecodeProvider* InTimecodeProvider, const BlackmagicDesign::FChannelInfo& InChannelInfo, const FFrameRate& InFrameRate)
			: RefCounter(0)
			, ChannelInfo(InChannelInfo)
			, State(ETimecodeProviderSynchronizationState::Closed)
			, FrameRate(InFrameRate)
			, bHasWarnedMissingTimecode(false)
			, TimecodeProvider(InTimecodeProvider)
		{
		}

		bool Initialize(const BlackmagicDesign::FInputChannelOptions& InChannelInfo)
		{
			AddRef();

			BlackmagicDesign::ReferencePtr<BlackmagicDesign::IInputEventCallback> SelfRef(this);
			BlackmagicIdendifier = BlackmagicDesign::RegisterCallbackForChannel(ChannelInfo, InChannelInfo, SelfRef);
			State = BlackmagicIdendifier.IsValid() ? ETimecodeProviderSynchronizationState::Synchronizing : ETimecodeProviderSynchronizationState::Error;
			return BlackmagicIdendifier.IsValid();
		}

		void Uninitialize()
		{
			if (BlackmagicIdendifier.IsValid())
			{
				BlackmagicDesign::UnregisterCallbackForChannel(ChannelInfo, BlackmagicIdendifier);
				BlackmagicIdendifier = BlackmagicDesign::FUniqueIdentifier();
			}

			Release();
		}

		ETimecodeProviderSynchronizationState GetSynchronizationState() const { return State; }

		FTimecode GetTimecode() const
		{
			FScopeLock Lock(&CallbackLock);
			return Timecode;
		}

	private:
		virtual void AddRef() override
		{
			++RefCounter;
		}

		virtual void Release() override
		{
			--RefCounter;
			if (RefCounter == 0)
			{
				delete this;
			}
		}

		virtual void OnInitializationCompleted(bool bSuccess) override
		{
			State = bSuccess ? ETimecodeProviderSynchronizationState::Synchronized : ETimecodeProviderSynchronizationState::Error;
		}

		virtual void OnShutdownCompleted() override
		{
			State = ETimecodeProviderSynchronizationState::Closed;
		}

		virtual void OnFrameReceived(const BlackmagicDesign::IInputEventCallback::FFrameReceivedInfo& InFrameInfo) override
		{
			FScopeLock Lock(&CallbackLock);
			if (InFrameInfo.bHaveTimecode)
			{
				//We expect the timecode to be processed in the library. What we receive will be a "linear" timecode even for frame rates greater than 30.
				if ((int32)InFrameInfo.Timecode.Frames >= FMath::RoundToInt(FrameRate.AsDecimal()))
				{
					UE_LOGF(LogBlackmagicMedia, Warning, "BlackmagicTimecodeProvider input '%d' received an invalid Timecode frame number (%d) for the current frame rate (%ls).", ChannelInfo.DeviceIndex, InFrameInfo.Timecode.Frames, *FrameRate.ToPrettyText().ToString());
				}

				Timecode = FTimecode(InFrameInfo.Timecode.Hours, InFrameInfo.Timecode.Minutes, InFrameInfo.Timecode.Seconds, InFrameInfo.Timecode.Frames, InFrameInfo.Timecode.bIsDropFrame);
			}
			else if (!bHasWarnedMissingTimecode)
			{
				bHasWarnedMissingTimecode = true;
				UE_LOGF(LogBlackmagicMedia, Warning, "BlackmagicTimecodeProvider input '%d' didn't receive any timecode in the last frame. Is your source configured correctly?", ChannelInfo.DeviceIndex);
			}
		}

		virtual void OnFrameFormatChanged(const BlackmagicDesign::FFormatInfo& NewFormat) override
		{
			if (UBlackmagicTimecodeProvider* TimecodeProviderPtr = TimecodeProvider.Get())
			{
				if (TimecodeProviderPtr->bAutoDetectTimecode)
				{
					FrameRate = FFrameRate(NewFormat.FrameRateNumerator, NewFormat.FrameRateDenominator);
					TimecodeProviderPtr->TimecodeConfiguration.MediaConfiguration.MediaMode.FrameRate = FrameRate;
					TimecodeProviderPtr->TimecodeConfiguration.MediaConfiguration.MediaMode.Resolution = FIntPoint(NewFormat.Width, NewFormat.Height);
					TimecodeProviderPtr->TimecodeConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier = NewFormat.DisplayMode;

					switch (NewFormat.FieldDominance)
					{
						case BlackmagicDesign::EFieldDominance::Progressive:
						{
							TimecodeProviderPtr->TimecodeConfiguration.MediaConfiguration.MediaMode.Standard = EMediaIOStandardType::Progressive;
							break;
						}
						case BlackmagicDesign::EFieldDominance::ProgressiveSegmentedFrame:
						{
							TimecodeProviderPtr->TimecodeConfiguration.MediaConfiguration.MediaMode.Standard = EMediaIOStandardType::ProgressiveSegmentedFrame;
							break;
						}
						case BlackmagicDesign::EFieldDominance::Interlaced:
						{
							TimecodeProviderPtr->TimecodeConfiguration.MediaConfiguration.MediaMode.Standard = EMediaIOStandardType::Interlaced;
							break;
						}
						default:
						{
							ensure(0);
							break;
						}
					}

					return;
				}
			}

			UE_LOGF(LogBlackmagicMedia, Error, "The video format changed.");
			State = ETimecodeProviderSynchronizationState::Error;
		}

		virtual void OnInterlacedOddFieldEvent(int64 FrameNumber) override
		{
			FScopeLock Lock(&CallbackLock);
			Timecode.Frames++;
		}

	private:
		TAtomic<int32> RefCounter;

		BlackmagicDesign::FUniqueIdentifier BlackmagicIdendifier;
		BlackmagicDesign::FChannelInfo ChannelInfo;

		mutable FCriticalSection CallbackLock;
		FTimecode Timecode;

		ETimecodeProviderSynchronizationState State;

		FFrameRate FrameRate;

		bool bHasWarnedMissingTimecode;

		TWeakObjectPtr<UBlackmagicTimecodeProvider> TimecodeProvider;
	};

}


//~ UBlackmagicTimecodeProvider implementation
//--------------------------------------------------------------------
UBlackmagicTimecodeProvider::UBlackmagicTimecodeProvider()
	: EventCallback(nullptr)
{
	TimecodeConfiguration.MediaConfiguration = FBlackmagicDeviceProvider().GetDefaultConfiguration();
	TimecodeConfiguration.MediaConfiguration.bIsInput = true;
}

bool UBlackmagicTimecodeProvider::FetchTimecode(FQualifiedFrameTime& OutFrameTime)
{
	if (!EventCallback || (EventCallback->GetSynchronizationState() != ETimecodeProviderSynchronizationState::Synchronized))
	{
		return false;
	}

	const FFrameRate Rate = TimecodeConfiguration.MediaConfiguration.MediaMode.FrameRate;
	const FTimecode Timecode = EventCallback->GetTimecode();

	OutFrameTime = FQualifiedFrameTime(Timecode, Rate);

	return true;
}

ETimecodeProviderSynchronizationState UBlackmagicTimecodeProvider::GetSynchronizationState() const
{
	return EventCallback ? EventCallback->GetSynchronizationState() : ETimecodeProviderSynchronizationState::Closed;
}

bool UBlackmagicTimecodeProvider::Initialize(class UEngine* InEngine)
{
	if (!TimecodeConfiguration.MediaConfiguration.IsValid())
	{
		UE_LOGF(LogBlackmagicMedia, Warning, "The configuration of '%ls' is not valid.", *GetName());
		return false;
	}

	if (!IBlackmagicMediaModule::Get().IsInitialized())
	{
		UE_LOGF(LogBlackmagicMedia, Error, "The TimecodeProvider '%ls' can't be initialized. Blackmagic is not initialized on your machine.", *GetName());
		return false;
	}

	if (TimecodeConfiguration.TimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None)
	{
		UE_LOGF(LogBlackmagicMedia, Error, "The TimecodeProvider '%ls' can't be initialized. Selected timecode format is invalid.", *GetName());
		return false;
	}

	BlackmagicDesign::FChannelInfo ChannelInfo;
	ChannelInfo.DeviceIndex = TimecodeConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier;

	check(EventCallback == nullptr);
	EventCallback = new BlackmagicTimecodeProviderHelpers::FEventCallback(this, ChannelInfo, TimecodeConfiguration.MediaConfiguration.MediaMode.FrameRate);

	BlackmagicDesign::FInputChannelOptions ChannelOptions;
	ChannelOptions.CallbackPriority = 5;
	ChannelOptions.FormatInfo.DisplayMode = TimecodeConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier;
	ChannelOptions.FormatInfo.FrameRateNumerator = TimecodeConfiguration.MediaConfiguration.MediaMode.FrameRate.Numerator;
	ChannelOptions.FormatInfo.FrameRateDenominator = TimecodeConfiguration.MediaConfiguration.MediaMode.FrameRate.Denominator;

	switch (TimecodeConfiguration.MediaConfiguration.MediaMode.Standard)
	{
	case EMediaIOStandardType::Interlaced:
		ChannelOptions.FormatInfo.FieldDominance = BlackmagicDesign::EFieldDominance::Interlaced;
		break;
	case EMediaIOStandardType::ProgressiveSegmentedFrame:
		ChannelOptions.FormatInfo.FieldDominance = BlackmagicDesign::EFieldDominance::ProgressiveSegmentedFrame;
		break;
	case EMediaIOStandardType::Progressive:
	default:
		ChannelOptions.FormatInfo.FieldDominance = BlackmagicDesign::EFieldDominance::Progressive;
		break;
	}

	if (bAutoDetectTimecode)
	{
		ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_Auto;
	}
	else
	{
		ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_None;
		switch (TimecodeConfiguration.TimecodeFormat)
		{
		case EMediaIOAutoDetectableTimecodeFormat::LTC:
			ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_LTC;
			break;
		case EMediaIOAutoDetectableTimecodeFormat::VITC:
			ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_VITC1;
			break;
		default:
			break;
		}
	}


	const bool bSuccess = EventCallback->Initialize(ChannelOptions);
	if (!bSuccess)
	{
		ReleaseResources();
	}

	return bSuccess;
}

void UBlackmagicTimecodeProvider::Shutdown(class UEngine* InEngine)
{
	ReleaseResources();
}

void UBlackmagicTimecodeProvider::BeginDestroy()
{
	ReleaseResources();
	Super::BeginDestroy();
}

void UBlackmagicTimecodeProvider::ReleaseResources()
{
	if (EventCallback)
	{
		EventCallback->Uninitialize();
		EventCallback = nullptr;
	}
}

void UBlackmagicTimecodeProvider::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (TimecodeFormat_DEPRECATED != EMediaIOTimecodeFormat::None)
	{
		TimecodeConfiguration.TimecodeFormat = UE::MediaIO::ToAutoDetectableTimecodeFormat(TimecodeFormat_DEPRECATED);
		TimecodeFormat_DEPRECATED = EMediaIOTimecodeFormat::None;
	}

	if (MediaConfiguration_DEPRECATED.IsValid())
	{
		TimecodeConfiguration.MediaConfiguration = MediaConfiguration_DEPRECATED;
		MediaConfiguration_DEPRECATED = FMediaIOConfiguration();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}
