// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicDeviceProvider.h"

#include "Algo/Sort.h"
#include "BlackmagicCoreModule.h"
#include "BlackmagicLib.h"
#include "BlackmagicMediaPrivate.h"
#include "BlackmagicMediaSource.h"
#include "IBlackmagicMediaModule.h"
#include "MediaIOCoreCommonDisplayMode.h"

#define LOCTEXT_NAMESPACE "BlackmagicDeviceProvider"



FName FBlackmagicDeviceProvider::GetProviderName()
{
	static FName NAME_Provider = "Blackmagic";
	return NAME_Provider;
}


FName FBlackmagicDeviceProvider::GetProtocolName()
{
	static FName NAME_Protocol = "blackmagicdesign"; // also defined in FBlackmagicMediaFactoryModule
	return NAME_Protocol;
}


namespace BlackmagicDeviceProvider
{
	FMediaIOMode ToMediaMode(const BlackmagicDesign::BlackmagicVideoFormats::VideoFormatDescriptor& InDescriptor)
	{
		FMediaIOMode MediaMode;
		MediaMode.Resolution = FIntPoint(InDescriptor.ResolutionWidth, InDescriptor.ResolutionHeight);
		MediaMode.Standard = EMediaIOStandardType::Progressive;
		if (InDescriptor.bIsInterlacedStandard)
		{
			MediaMode.Standard = EMediaIOStandardType::Interlaced;
		}
		else if (InDescriptor.bIsPsfStandard)
		{
			MediaMode.Standard = EMediaIOStandardType::ProgressiveSegmentedFrame;
		}

		MediaMode.FrameRate = FFrameRate(InDescriptor.FrameRateNumerator, InDescriptor.FrameRateDenominator);
		MediaMode.DeviceModeIdentifier = InDescriptor.VideoFormatIndex;

		if (InDescriptor.bIsInterlacedStandard)
		{
			MediaMode.FrameRate.Numerator *= 2;
		}

		return MediaMode;
	}

	bool IsVideoFormatValid(const BlackmagicDesign::BlackmagicVideoFormats::VideoFormatDescriptor& InDescriptor)
	{
		if (!InDescriptor.bIsValid)
		{
			return false;
		}
		if (InDescriptor.bIsPsfStandard /*|| InDescriptor.bIsVideoFormatB*/)
		{
			return false;
		}
		if (InDescriptor.bIsSD)
		{
			return false;
		}
		return true;
	}
}


FName FBlackmagicDeviceProvider::GetFName()
{
	return GetProviderName();
}


TArray<FMediaIOConnection> FBlackmagicDeviceProvider::GetConnections() const
{
	TArray<FMediaIOConnection> Results;

	TArray<FMediaIODevice> Devices = GetDevices();
	Results.Reserve(Devices.Num());
	const FMediaIOConnection DefaultConnection = GetDefaultConfiguration().MediaConnection;
	for(const FMediaIODevice& Device : Devices)
	{
		FMediaIOConnection Connection = DefaultConnection;
		Connection.Device = Device;
		Results.Add(Connection);
	}

	return MoveTemp(Results);
}


TArray<FMediaIOConfiguration> FBlackmagicDeviceProvider::GetConfigurations() const
{
	return GetConfigurations(true, true);
}


TArray<FMediaIOConfiguration> FBlackmagicDeviceProvider::GetConfigurations(bool bAllowInput, bool bAllowOutput) const
{
	const int32 MaxNumberOfChannel = 8;

	TArray<FMediaIOConfiguration> Results;
	
	if (!IBlackmagicMediaModule::Get().IsInitialized())
	{
		return Results;
	}

	BlackmagicDesign::BlackmagicDeviceScanner DeviceScanner;
	int32 NumDevices = DeviceScanner.GetNumDevices();
	for (int32 DeviceIndex = 1; DeviceIndex <= NumDevices; ++DeviceIndex)
	{
		BlackmagicDesign::BlackmagicDeviceScanner::FormatedTextType DeviceNameBuffer;
		if (DeviceScanner.GetDeviceTextId(DeviceIndex, DeviceNameBuffer))
		{
			BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo DeviceInfo;
			if (DeviceScanner.GetDeviceInfo(DeviceIndex, DeviceInfo))
			{
				if (!DeviceInfo.bIsSupported)
				{
					continue;
				}

				if (bAllowInput && !DeviceInfo.bCanDoCapture)
				{
					continue;
				}

				if (bAllowOutput && !DeviceInfo.bCanDoPlayback)
				{
					continue;
				}

				FMediaIOConfiguration MediaConfiguration;
				MediaConfiguration.MediaConnection.Device.DeviceIdentifier = DeviceIndex;
				MediaConfiguration.MediaConnection.Device.DeviceName = FName(DeviceNameBuffer);
				MediaConfiguration.MediaConnection.Protocol = FBlackmagicDeviceProvider::GetProtocolName();
				MediaConfiguration.MediaConnection.PortIdentifier = 0;

				auto BuildList = [&](bool bIsInput)
				{
					MediaConfiguration.bIsInput = bIsInput;

					BlackmagicDesign::BlackmagicVideoFormats FrameFormats(DeviceIndex, !bIsInput);
					const int32 NumSupportedFormat = FrameFormats.GetNumSupportedFormat();
					for (int32 FormatIndex = 0; FormatIndex < NumSupportedFormat; ++FormatIndex)
					{
						BlackmagicDesign::BlackmagicVideoFormats::VideoFormatDescriptor Descriptor = FrameFormats.GetSupportedFormat(FormatIndex);
						if (!BlackmagicDeviceProvider::IsVideoFormatValid(Descriptor))
						{
							continue;
						}

						MediaConfiguration.MediaMode = BlackmagicDeviceProvider::ToMediaMode(Descriptor);

						MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::SingleLink;
						MediaConfiguration.MediaConnection.QuadTransportType = EMediaIOQuadLinkTransportType::TwoSampleInterleave;
						Results.Add(MediaConfiguration);

						// The SDK doesn't say so but all devices that can do Dual|QuadLink can do SingleLink 12G.
						//The card auto detect it in input. We need to tell it in output.
						if (!MediaConfiguration.bIsInput)
						{
							if (Descriptor.bIs2K)
							{
								if (DeviceInfo.bCanDoDualLink)
								{
									MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::DualLink;
									Results.Add(MediaConfiguration);
								}
							}
							else if (Descriptor.bIs4K)
							{
								if (DeviceInfo.bCanDoQuadLink)
								{
									MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::QuadLink;
									MediaConfiguration.MediaConnection.QuadTransportType = EMediaIOQuadLinkTransportType::TwoSampleInterleave;
									Results.Add(MediaConfiguration);

									if (DeviceInfo.bCanDoQuadSquareLink)
									{
										MediaConfiguration.MediaConnection.QuadTransportType = EMediaIOQuadLinkTransportType::SquareDivision;
										Results.Add(MediaConfiguration);
									}
								}
							}
						}
					}
				};

				if (bAllowInput)
				{
					BuildList(true);
				}

				if (bAllowOutput)
				{
					BuildList(false);
				}
			}
		}
	}

	Algo::SortBy(Results, [](const FMediaIOConfiguration& Config){ return Config.MediaConnection.Device.DeviceName; }, FNameLexicalLess());

	return Results;
}



TArray<FMediaIOInputConfiguration> FBlackmagicDeviceProvider::GetInputConfigurations() const
{
	return TArray<FMediaIOInputConfiguration>();
}

TArray<FMediaIOOutputConfiguration> FBlackmagicDeviceProvider::GetOutputConfigurations() const
{
	TArray<FMediaIOOutputConfiguration> Results;

	TArray<FMediaIOConfiguration> OutputConfigurations = GetConfigurations(false, true);
	TArray<FMediaIOConnection> OtherConnections = GetConnections();

	Results.Reset(OutputConfigurations.Num() * 4);

	BlackmagicDesign::BlackmagicDeviceScanner DeviceScanner;
	int32 LastDeviceIdentifier = INDEX_NONE;
	bool bCanDoKeyAndFill = false;
	bool bCanDoExternal = false;
	FMediaIOOutputConfiguration DefaultOutputConfiguration = GetDefaultOutputConfiguration();
	DefaultOutputConfiguration.KeyPortIdentifier = 0;

	for (const FMediaIOConfiguration& OutputConfiguration : OutputConfigurations)
	{
		auto BuildList = [&]()
		{
			DefaultOutputConfiguration.MediaConfiguration = OutputConfiguration;

			if (bCanDoExternal)
			{
				DefaultOutputConfiguration.OutputReference = EMediaIOReferenceType::External;
				Results.Add(DefaultOutputConfiguration);
			}

			DefaultOutputConfiguration.OutputReference = EMediaIOReferenceType::FreeRun;
			Results.Add(DefaultOutputConfiguration);
		};

		// Update the Device Info
		if (OutputConfiguration.MediaConnection.Device.DeviceIdentifier != LastDeviceIdentifier)
		{
			BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo DeviceInfo;
			if (!DeviceScanner.GetDeviceInfo(OutputConfiguration.MediaConnection.Device.DeviceIdentifier, DeviceInfo))
			{
				continue;
			}

			bCanDoKeyAndFill = DeviceInfo.bSupportExternalKeying;
			bCanDoExternal = DeviceInfo.bHasGenlockReferenceInput;
			LastDeviceIdentifier = OutputConfiguration.MediaConnection.Device.DeviceIdentifier;
		}

		// Build the list for fill only
		DefaultOutputConfiguration.OutputType = EMediaIOOutputType::Fill;
		BuildList();

		// Add all output port for key
		if (bCanDoKeyAndFill)
		{
			DefaultOutputConfiguration.OutputType = EMediaIOOutputType::FillAndKey;
			BuildList();
		}
	}

	return Results;
}

TArray<FMediaIOVideoTimecodeConfiguration> FBlackmagicDeviceProvider::GetTimecodeConfigurations() const
{
	TArray<FMediaIOVideoTimecodeConfiguration> MediaConfigurations;
	bool bHasInputConfiguration = false;
	{
		TArray<FMediaIOConfiguration> InputConfigurations = GetConfigurations(true, false);

		FMediaIOVideoTimecodeConfiguration DefaultTimecodeConfiguration;
		MediaConfigurations.Reset(InputConfigurations.Num() * 2);
		for (const FMediaIOConfiguration& InputConfiguration : InputConfigurations)
		{
			DefaultTimecodeConfiguration.MediaConfiguration = InputConfiguration;
			DefaultTimecodeConfiguration.TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::LTC;
			MediaConfigurations.Add(DefaultTimecodeConfiguration);

			DefaultTimecodeConfiguration.TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::VITC;
			MediaConfigurations.Add(DefaultTimecodeConfiguration);
		}
	}
	return MediaConfigurations;
}

TArray<FMediaIODevice> FBlackmagicDeviceProvider::GetDevices() const
{
	TArray<FMediaIODevice> Results;
	if (!IBlackmagicMediaModule::Get().IsInitialized())
	{
		return Results;
	}

	BlackmagicDesign::BlackmagicDeviceScanner DeviceScanner;
	int32 NumDevices = DeviceScanner.GetNumDevices();
	for (int32 DeviceIndex = 1; DeviceIndex <= NumDevices; ++DeviceIndex)
	{
		BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo DeviceInfo;
		if (!DeviceScanner.GetDeviceInfo(DeviceIndex, DeviceInfo))
		{
			continue;
		}

		if (!DeviceInfo.bIsSupported)
		{
			continue;
		}

		TCHAR DeviceNameBuffer[BlackmagicDesign::BlackmagicDeviceScanner::FormatedTextSize];
		if (!DeviceScanner.GetDeviceTextId(DeviceIndex, DeviceNameBuffer))
		{
			continue;
		}

		FMediaIODevice Device;
		Device.DeviceIdentifier = DeviceIndex;
		Device.DeviceName = DeviceNameBuffer;
		Results.Add(Device);
	}

	return MoveTemp(Results);
}


TArray<FMediaIOMode> FBlackmagicDeviceProvider::GetModes(const FMediaIODevice& InDevice, bool bInOutput) const
{
	TArray<FMediaIOMode> Results;

	if (FBlackmagicCoreModule* CoreModule = FModuleManager::GetModulePtr<FBlackmagicCoreModule>("BlackmagicCore"))
	{
		if (!CoreModule->IsInitialized() || !InDevice.IsValid() || !CoreModule->CanUseBlackmagicCard())
		{
			return Results;
		}
	}
	else
	{
		return Results;
	}

	BlackmagicDesign::BlackmagicDeviceScanner DeviceScanner;
	BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo DeviceInfo;
	if (!DeviceScanner.GetDeviceInfo(InDevice.DeviceIdentifier, DeviceInfo))
	{
		return Results;
	}

	if (!DeviceInfo.bIsSupported)
	{
		return Results;
	}

	if ((bInOutput && !DeviceInfo.bCanDoPlayback) || (!bInOutput && !DeviceInfo.bCanDoCapture))
	{
		return Results;
	}

	BlackmagicDesign::BlackmagicVideoFormats FrameFormats(InDevice.DeviceIdentifier, bInOutput);
	const int32 NumSupportedFormat = FrameFormats.GetNumSupportedFormat();
	Results.Reserve(NumSupportedFormat);
	for (int32 Index = 0; Index < NumSupportedFormat; ++Index)
	{
		BlackmagicDesign::BlackmagicVideoFormats::VideoFormatDescriptor Descriptor = FrameFormats.GetSupportedFormat(Index);
		if (!BlackmagicDeviceProvider::IsVideoFormatValid(Descriptor))
		{
			continue;
		}
		Results.Add(BlackmagicDeviceProvider::ToMediaMode(Descriptor));
	}

	return Results;
}


FMediaIOConfiguration FBlackmagicDeviceProvider::GetDefaultConfiguration() const
{
	FMediaIOConfiguration Configuration;
	Configuration.bIsInput = true;
	Configuration.MediaConnection.Device.DeviceIdentifier = 1;
	Configuration.MediaConnection.Protocol = GetProtocolName();
	Configuration.MediaConnection.PortIdentifier = 0;
	Configuration.MediaMode = GetDefaultMode();
	return Configuration;
}

FMediaIOVideoTimecodeConfiguration FBlackmagicDeviceProvider::GetDefaultTimecodeConfiguration() const
{
	FMediaIOVideoTimecodeConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	return Configuration;
}

FMediaIOMode FBlackmagicDeviceProvider::GetDefaultMode() const
{
	FMediaIOMode Mode;
	Mode.DeviceModeIdentifier = BlackmagicMediaOption::DefaultVideoFormat;
	Mode.FrameRate = FFrameRate(30, 1);
	Mode.Resolution = FIntPoint(1920, 1080);
	Mode.Standard = EMediaIOStandardType::Progressive;
	return Mode;
}


FMediaIOInputConfiguration FBlackmagicDeviceProvider::GetDefaultInputConfiguration() const
{
	FMediaIOInputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = true;
	Configuration.InputType = EMediaIOInputType::Fill;
	return Configuration;
}

FMediaIOOutputConfiguration FBlackmagicDeviceProvider::GetDefaultOutputConfiguration() const
{
	FMediaIOOutputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = false;
	Configuration.OutputReference = EMediaIOReferenceType::FreeRun;
	Configuration.OutputType = EMediaIOOutputType::Fill;
	return Configuration;
}


UMediaSource* FBlackmagicDeviceProvider::CreateMediaSource(
	const FMediaIOConfiguration& InConfiguration, UObject* Outer) const
{
	UBlackmagicMediaSource* MediaSource = NewObject<UBlackmagicMediaSource>(Outer);
	if (MediaSource != nullptr)
	{
		MediaSource->MediaConfiguration = InConfiguration;
	}

	return MediaSource;
}


FText FBlackmagicDeviceProvider::ToText(const FMediaIOConfiguration& InConfiguration, bool bInIsAutoDetected) const
{
	if (bInIsAutoDetected)
	{
		return FText::Format(LOCTEXT("FMediaIOConfigurationToTextAutoDetect", "[{0}] - {1} [device{2}/auto]")
		, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
		, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
		, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
		);
	}
	
	if (InConfiguration.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConfigurationToText", "[{0}] - {1} [device{2}/{3}]")
			, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
			, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
			, InConfiguration.MediaMode.GetModeName()
			);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}


FText FBlackmagicDeviceProvider::ToText(const FMediaIOConnection& InConnection) const
{
	if (InConnection.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConnectionToText", "{0} [device{1}]")
			, FText::FromName(InConnection.Device.DeviceName)
			, LOCTEXT("Device", "device")
			, FText::AsNumber(InConnection.Device.DeviceIdentifier)
			);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

FText FBlackmagicDeviceProvider::ToText(const FMediaIOOutputConfiguration& InConfiguration) const
{
	if (InConfiguration.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOOutputConfigurationToText", "{0} - {1} [device{2}/{3}/{4}]")
			, InConfiguration.OutputType == EMediaIOOutputType::Fill ? LOCTEXT("Fill", "Fill") : LOCTEXT("FillAndKey", "Fill&Key")
			, FText::FromName(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier)
			, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
			, InConfiguration.MediaConfiguration.MediaMode.GetModeName()
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

#undef LOCTEXT_NAMESPACE
