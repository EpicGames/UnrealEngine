// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicMediaDefinitions.h"

#include "BlackmagicLib.h"

namespace UE::BlackmagicMedia
{
	FBlackmagicMediaHDROptions MakeBlackmagicMediaHDROptions(const BlackmagicDesign::FHDRMetaData& HDROptions)
	{
		FBlackmagicMediaHDROptions HDRMetadata;

		switch (HDROptions.ColorSpace)
		{
		case BlackmagicDesign::EHDRMetaDataColorspace::Rec601:
			// Fallthrough
		case BlackmagicDesign::EHDRMetaDataColorspace::Rec709:
			HDRMetadata.Gamut = EBlackmagicHDRMetadataGamut::Rec709;
			break;
		case BlackmagicDesign::EHDRMetaDataColorspace::Rec2020:
			HDRMetadata.Gamut = EBlackmagicHDRMetadataGamut::Rec2020;
			break;
		}

		HDRMetadata.EOTF = (EBlackmagicHDRMetadataEOTF)HDROptions.EOTF;

		return HDRMetadata;
	}

	BlackmagicDesign::FHDRMetaData MakeBlackmagicHDROptions(const FBlackmagicMediaHDROptions& HDROptions)
	{
		BlackmagicDesign::FHDRMetaData HDRMetadata;

		switch (HDROptions.Gamut)
		{
		case EBlackmagicHDRMetadataGamut::Rec709:
			HDRMetadata.ColorSpace = BlackmagicDesign::EHDRMetaDataColorspace::Rec709;
			break;
		case EBlackmagicHDRMetadataGamut::Rec2020:
			HDRMetadata.ColorSpace = BlackmagicDesign::EHDRMetaDataColorspace::Rec2020;
			break;
		}

		HDRMetadata.EOTF = (BlackmagicDesign::EHDRMetaDataEOTF)HDROptions.EOTF;

		return HDRMetadata;
	}
}
