// Copyright Epic Games, Inc. All Rights Reserved.

#include "IBlackmagicMediaModule.h"

#include "BlackmagicCoreModule.h"
#include "BlackmagicDeviceProvider.h"
#include "BlackmagicMediaPrivate.h"
#include "Player/BlackmagicMediaPlayer.h"
#include "Brushes/SlateImageBrush.h"
#include "IMediaIOCoreModule.h"
#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyle.h"


DEFINE_LOG_CATEGORY(LogBlackmagicMedia);

#define LOCTEXT_NAMESPACE "BlackmagicMediaModule"

/**
 * Implements the NdiMedia module.
 */
class FBlackmagicMediaModule : public IBlackmagicMediaModule
{
public:

	//~ IBlackmagicMediaModule interface
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		
		if (FBlackmagicCoreModule* CoreModule = FModuleManager::GetModulePtr<FBlackmagicCoreModule>("BlackmagicCore"))
		{
			if (!CoreModule->IsInitialized())
			{
				return nullptr;
			}
		}
		else
		{
			return nullptr;
		}

		return MakeShared<FBlackmagicMediaPlayer, ESPMode::ThreadSafe>(EventSink);
	}

	virtual TSharedPtr<FSlateStyleSet> GetStyle() override
	{
		return StyleSet;
	}

	virtual bool IsInitialized() const override
	{
		if (FBlackmagicCoreModule* CoreModule = FModuleManager::GetModulePtr<FBlackmagicCoreModule>("BlackmagicCore"))
		{
			return CoreModule->IsInitialized();
		}

		return false;
	}

	virtual bool CanBeUsed() const override 
	{
		if (FBlackmagicCoreModule* CoreModule = FModuleManager::GetModulePtr<FBlackmagicCoreModule>("BlackmagicCore"))
		{
			return CoreModule->CanUseBlackmagicCard();
		}
		return false;
	}

public:

	//~ IModuleInterface interface
	virtual void StartupModule() override
	{
		// initialize
		CreateStyle();
		
		IMediaIOCoreModule::Get().RegisterDeviceProvider(&DeviceProvider);
	}

	virtual void ShutdownModule() override
	{
		if (IMediaIOCoreModule::IsAvailable())
		{
			IMediaIOCoreModule::Get().UnregisterDeviceProvider(&DeviceProvider);
		}
	}

private:
	void CreateStyle()
	{
		static FName StyleName(TEXT("BlackmagicMediaStyle"));
		StyleSet = MakeShared<FSlateStyleSet>(StyleName);

		const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("BlackmagicMedia"))->GetContentDir();
		const FVector2D Icon16x16(16.0f, 16.0f);
		StyleSet->Set("BlackmagicMediaIcon", new FSlateImageBrush((ContentDir / TEXT("Editor/Icons/BlackmagicMediaSource_64x")) + TEXT(".png"), Icon16x16));
	}

private:
	FBlackmagicDeviceProvider DeviceProvider;
	TSharedPtr<FSlateStyleSet> StyleSet;
};

IMPLEMENT_MODULE(FBlackmagicMediaModule, BlackmagicMedia);

#undef LOCTEXT_NAMESPACE
