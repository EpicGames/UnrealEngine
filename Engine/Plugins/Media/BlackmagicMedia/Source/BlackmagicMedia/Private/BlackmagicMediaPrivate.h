// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlackmagicLib.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBlackmagicMedia, Log, All);

namespace BlackmagicMediaOption
{
	static const FName DeviceIndex("DeviceIndex");
	static const FName TimecodeFormat("TimecodeFormat");
	static const FName CaptureAudio("CaptureAudio");
	static const FName AudioChannelOption("AudioChannel");
	static const FName MaxAudioFrameBuffer("MaxAudioFrameBuffer");
	static const FName CaptureVideo("CaptureVideo");
	static const FName BlackmagicVideoFormat("BlackmagicVideoFormat");
	static const FName ColorFormat("ColorFormat");
	static const FName MaxVideoFrameBuffer("MaxVideoFrameBuffer");
	static const FName LogDropFrame("LogDropFrame");
	static const FName EncodeTimecodeInTexel("EncodeTimecodeInTexel");
	static const FName SRGBInput("sRGBInput");

	static const BlackmagicDesign::FBlackmagicVideoFormat DefaultVideoFormat = 0x48703330; //1080p 30fps

}