// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/BlackmagicMediaPlayer.h"

#include "BlackmagicMediaDefinitions.h"

#include "Engine/GameEngine.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "IBlackmagicMediaModule.h"
#include "IMediaEventSink.h"
#include "IMediaIOCoreModule.h"
#include "IMediaOptions.h"
#include "MediaIOCoreDeinterlacer.h"
#include "MediaIOCoreEncodeTime.h"
#include "MediaIOCoreFileWriter.h"
#include "MediaIOCoreSamples.h"
#include "MediaIOCoreTextureSampleBase.h"
#include "MediaIOCoreUtilities.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "OpenColorIOColorSpace.h"
#include "RenderCommandFence.h"
#include "Slate/SceneViewport.h"
#include "Stats/Stats.h"
#include "Styling/SlateStyle.h"
#include "Templates/Atomic.h"

#include "Async/Async.h"
#include "CaptureCardMediaSource.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif


#define LOCTEXT_NAMESPACE "BlackmagicMediaPlayer"

DECLARE_CYCLE_STAT(TEXT("Blackmagic MediaPlayer Process received frame"), STAT_Blackmagic_MediaPlayer_ProcessReceivedFrame, STATGROUP_Media);

bool bBlackmagicWriteOutputRawDataCmdEnable = false;
static FAutoConsoleCommand BlackmagicWriteOutputRawDataCmd(
	TEXT("Blackmagic.WriteOutputRawData"),
	TEXT("Write Blackmagic raw output buffer to file."),
	FConsoleCommandDelegate::CreateLambda([]() { bBlackmagicWriteOutputRawDataCmdEnable = true;	})
	);

namespace BlackmagicMediaPlayerHelpers
{
	static const int32 ToleratedExtraMaxBufferCount = 2;

	class FBlackmagicMediaPlayerEventCallback : public BlackmagicDesign::IInputEventCallback
	{
	public:
		FBlackmagicMediaPlayerEventCallback(FBlackmagicMediaPlayer* InMediaPlayer, const BlackmagicDesign::FChannelInfo& InChannelInfo)
			: RefCounter(0)
			, ChannelInfo(InChannelInfo)
			, MediaPlayer(InMediaPlayer)
			, MediaState(EMediaState::Closed)
			, PrevousTimespan(FTimespan::Zero())
			, bEncodeTimecodeInTexel(false)
			, LastBitsPerSample(0)
			, LastNumChannels(0)
			, LastSampleRate(0)
			, PreviousAudioFrameDropCount(0)
			, PreviousVideoFrameDropCount(0)
			, LastHasFrameTime(0.0)
			, bReceivedValidFrame(false)
			, bIsTimecodeExpected(false)
			, bHasWarnedMissingTimecode(false)
		{
		}

		bool Initialize(const BlackmagicDesign::FInputChannelOptions& InChannelInfo, bool bInEncodeTimecodeInTexel, int32 InMaxNumAudioFrameBuffer, int32 InMaxNumVideoFrameBuffer)
		{
			AddRef();

			bEncodeTimecodeInTexel = bInEncodeTimecodeInTexel;
			MaxNumAudioFrameBuffer = InMaxNumAudioFrameBuffer;
			MaxNumVideoFrameBuffer = InMaxNumVideoFrameBuffer;
			bIsTimecodeExpected = InChannelInfo.TimecodeFormat != BlackmagicDesign::ETimecodeFormat::TCF_None;

			BlackmagicDesign::ReferencePtr<BlackmagicDesign::IInputEventCallback> SelfRef(this);
			BlackmagicIdendifier = BlackmagicDesign::RegisterCallbackForChannel(ChannelInfo, InChannelInfo, SelfRef);
			MediaState = BlackmagicIdendifier.IsValid() ? EMediaState::Preparing : EMediaState::Error;
			return BlackmagicIdendifier.IsValid();
		}

		void Uninitialize()
		{
			FScopeLock Lock(&CallbackLock);
			MediaPlayer = nullptr;

			if (BlackmagicIdendifier.IsValid())
			{
				MediaState = EMediaState::Stopped;
				BlackmagicDesign::UnregisterCallbackForChannel(ChannelInfo, BlackmagicIdendifier);
				BlackmagicIdendifier = BlackmagicDesign::FUniqueIdentifier();
			}

			Release();
		}

		EMediaState GetMediaState() const { return MediaState; }

		void UpdateAudioTrackFormat(FMediaAudioTrackFormat& OutAudioTrackFormat)
		{
			OutAudioTrackFormat.BitsPerSample = LastBitsPerSample;
			OutAudioTrackFormat.NumChannels = LastNumChannels;
			OutAudioTrackFormat.SampleRate = LastSampleRate;
		}

		void VerifyFrameDropCount_GameThread(const FString& InUrl)
		{
			if (MediaPlayer->bVerifyFrameDropCount)
			{
				const int32 CurrentAudioDropCount = MediaPlayer->Samples->GetAudioFrameDropCount();
				int32 DeltaAudioDropCount = CurrentAudioDropCount;
				if (CurrentAudioDropCount >= PreviousAudioFrameDropCount)
				{
					DeltaAudioDropCount = CurrentAudioDropCount - PreviousAudioFrameDropCount;
				}
				PreviousAudioFrameDropCount = CurrentAudioDropCount;
				if (DeltaAudioDropCount > 0)
				{
					UE_LOGF(LogBlackmagicMedia, Warning, "Lost %d audio frames on input %ls. Frame rate is either too slow or buffering capacity is too small.", DeltaAudioDropCount, *InUrl);
				}

				// JITR pipeline always keeps the sample pool full. So every new sample increments internal drop count.
				// This if-condition is intended to avoid "Lost %d XXX frames on input..." message spam every frame.
				if (!MediaPlayer->IsJustInTimeRenderingEnabled())
				{
					const int32 CurrentVideoDropCount = MediaPlayer->Samples->GetVideoFrameDropCount();
					int32 DeltaVideoDropCount = CurrentVideoDropCount;
					if (CurrentVideoDropCount >= PreviousVideoFrameDropCount)
					{
						DeltaVideoDropCount = CurrentVideoDropCount - PreviousVideoFrameDropCount;
					}
					PreviousVideoFrameDropCount = CurrentVideoDropCount;
					if (DeltaVideoDropCount > 0)
					{
						UE_LOGF(LogBlackmagicMedia, Warning, "Lost %d video frames on input %ls. Frame rate is either too slow or buffering capacity is too small.", DeltaVideoDropCount, *InUrl);
					}
				}
			}
		}

	private:
		virtual void AddRef() override
		{
			++RefCounter;
		}

		virtual void Release() override
		{
			--RefCounter;
			if (RefCounter == 0)
			{
				delete this;
			}
		}

		virtual void OnInitializationCompleted(bool bSuccess) override
		{
			MediaState = bSuccess ? EMediaState::Playing : EMediaState::Error;
		}

		virtual void OnShutdownCompleted() override
		{
			MediaState = EMediaState::Closed;
		}

		virtual void OnFrameReceived(
			const BlackmagicDesign::IInputEventCallback::FFrameReceivedInfo& InFrameInfo,
			BlackmagicDesign::IInputEventCallback::FFrameReceivedBufferHolders& OutBufferHolders) override
		{
			SCOPE_CYCLE_COUNTER(STAT_Blackmagic_MediaPlayer_ProcessReceivedFrame);

			FScopeLock Lock(&CallbackLock);

			if (MediaPlayer == nullptr)
			{
				return;
			}

			if (!InFrameInfo.bHasInputSource && InFrameInfo.AudioBuffer == nullptr)
			{
				const double CurrentTime = FPlatformTime::Seconds();
				const double TimeAllowedToConnect = 2.0;
				if (LastHasFrameTime < 0.1)
				{
					LastHasFrameTime = CurrentTime;
				}
				if (bReceivedValidFrame || CurrentTime - LastHasFrameTime > TimeAllowedToConnect)
				{
					if (!MediaPlayer->bAutoDetect)
					{
						UE_LOGF(LogBlackmagicMedia, Error, "There is no video input for '%ls'.", *MediaPlayer->GetUrl());
						MediaState = EMediaState::Error;
					}
					else
					{
						MediaState = EMediaState::Paused;
					}
				}
				return;
			}
			else if (MediaState == EMediaState::Paused)
			{
				MediaState = EMediaState::Playing;
			}

			bReceivedValidFrame = bReceivedValidFrame || InFrameInfo.bHasInputSource;

			FTimespan DecodedTime = FTimespan::FromSeconds(MediaPlayer->GetPlatformSeconds());
			FTimespan DecodedTimeF2 = DecodedTime + FTimespan::FromSeconds(MediaPlayer->VideoFrameRate.AsInterval());

			if (MediaState == EMediaState::Playing)
			{
				TOptional<FTimecode> DecodedTimecode;
				TOptional<FTimecode> DecodedTimecodeF2;

				if (InFrameInfo.bHaveTimecode)
				{
					//We expect the timecode to be processed in the library. What we receive will be a "linear" timecode even for frame rates greater than 30.
					const int32 FrameLimit = InFrameInfo.FieldDominance != BlackmagicDesign::EFieldDominance::Interlaced ? FMath::RoundToInt(MediaPlayer->VideoFrameRate.AsDecimal()) : FMath::RoundToInt(MediaPlayer->VideoFrameRate.AsDecimal()) - 1;
					if ((int32)InFrameInfo.Timecode.Frames >= FrameLimit)
					{
						UE_LOGF(LogBlackmagicMedia, Warning, "Input '%ls' received an invalid Timecode frame number (%d) for the current frame rate (%ls).", *MediaPlayer->GetUrl(), InFrameInfo.Timecode.Frames, *MediaPlayer->VideoFrameRate.ToPrettyText().ToString());
					}

					DecodedTimecode = FTimecode(InFrameInfo.Timecode.Hours, InFrameInfo.Timecode.Minutes, InFrameInfo.Timecode.Seconds, InFrameInfo.Timecode.Frames, InFrameInfo.Timecode.bIsDropFrame);
					DecodedTimecodeF2 = DecodedTimecode;
					++DecodedTimecodeF2->Frames;

					const FFrameNumber ConvertedFrameNumber = DecodedTimecode.GetValue().ToFrameNumber(MediaPlayer->VideoFrameRate);
					const double NumberOfSeconds = ConvertedFrameNumber.Value * MediaPlayer->VideoFrameRate.AsInterval();
					const FTimespan TimecodeDecodedTime = FTimespan::FromSeconds(NumberOfSeconds);
					if (MediaPlayer->EvaluationType == EMediaIOSampleEvaluationType::Timecode)
					{
						DecodedTime = TimecodeDecodedTime;
						DecodedTimeF2 = TimecodeDecodedTime + FTimespan::FromSeconds(MediaPlayer->VideoFrameRate.AsInterval());
					}

					PreviousTimecode = InFrameInfo.Timecode;
					PrevousTimespan = TimecodeDecodedTime;

					if (MediaPlayer->IsTimecodeLogEnabled())
					{
						UE_LOGF(LogBlackmagicMedia, Log, "Input '%ls' has timecode : %02d:%02d:%02d:%02d", *MediaPlayer->GetUrl()
							, InFrameInfo.Timecode.Hours, InFrameInfo.Timecode.Minutes, InFrameInfo.Timecode.Seconds, InFrameInfo.Timecode.Frames);
					}
				}
				else if (!bHasWarnedMissingTimecode && bIsTimecodeExpected)
				{
					bHasWarnedMissingTimecode = true;
					UE_LOGF(LogBlackmagicMedia, Warning, "Input '%ls' is expecting timecode but didn't receive any in the last frame. Is your source configured correctly?", *MediaPlayer->GetUrl());
				}

				if (InFrameInfo.AudioBuffer)
				{
					auto AudioSamle = MediaPlayer->AudioSamplePool->AcquireShared();
					if (AudioSamle->Initialize(reinterpret_cast<int32*>(InFrameInfo.AudioBuffer)
						, InFrameInfo.AudioBufferSize / sizeof(int32)
						, InFrameInfo.NumberOfAudioChannel
						, InFrameInfo.AudioRate
						, DecodedTime
						, DecodedTimecode))
					{
						MediaPlayer->AddAudioSample(AudioSamle);

						LastBitsPerSample = sizeof(int32);
						LastSampleRate = InFrameInfo.AudioRate;
						LastNumChannels = InFrameInfo.NumberOfAudioChannel;
					}
				}

				if (InFrameInfo.VideoBuffer)
				{
					const bool bIsProgressivePicture = InFrameInfo.FieldDominance == BlackmagicDesign::EFieldDominance::Progressive;
					EMediaTextureSampleFormat SampleFormat = EMediaTextureSampleFormat::CharBGRA;
					EMediaIOCoreEncodePixelFormat EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharUYVY;
					FString OutputFilename = "";

					switch (InFrameInfo.PixelFormat)
					{
					case BlackmagicDesign::EPixelFormat::pf_8Bits:
						SampleFormat = EMediaTextureSampleFormat::CharUYVY;
						EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharUYVY;
						OutputFilename = FString::Printf(TEXT("Blackmagic_Output_8_YUV_ch%d"), ChannelInfo.DeviceIndex);
						break;
					case BlackmagicDesign::EPixelFormat::pf_10Bits:
						SampleFormat = EMediaTextureSampleFormat::YUVv210;
						EncodePixelFormat = EMediaIOCoreEncodePixelFormat::YUVv210;
						OutputFilename = FString::Printf(TEXT("Blackmagic_Output_10_YUV_ch%d"), ChannelInfo.DeviceIndex);
						break;
					}

					if (bBlackmagicWriteOutputRawDataCmdEnable)
					{
						MediaIOCoreFileWriter::WriteRawFile(OutputFilename, reinterpret_cast<uint8*>(InFrameInfo.VideoBuffer), InFrameInfo.VideoPitch * InFrameInfo.VideoHeight);
						bBlackmagicWriteOutputRawDataCmdEnable = false;
					}
					
					FBlackmagicMediaHDROptions HDROptions = UE::BlackmagicMedia::MakeBlackmagicMediaHDROptions(InFrameInfo.HDRMetaData);
					ensure(MediaPlayer->BlackmagicColorSettings.IsValid());
		
					const UE::Color::EEncoding SourceEncoding = MediaPlayer->SourceColorSettings.IsValid() ? MediaPlayer->SourceColorSettings->GetEncodingOverride() : UE::Color::EEncoding::None;
					if (SourceEncoding != UE::Color::EEncoding::None)
					{
						// Explicit user encoding override: propagate both encoding and its scoped
						// reference white so the user's UI selection reaches the sample.
						MediaPlayer->BlackmagicColorSettings->SetEncodingOverride(SourceEncoding);
						MediaPlayer->BlackmagicColorSettings->SetReferenceWhiteOverride(MediaPlayer->SourceColorSettings->GetReferenceWhiteOverride());
					}
					else
					{
						const FTimespan TimeBetweenLogs = FTimespan::FromSeconds(5);

						switch (HDROptions.EOTF)
						{
						case EBlackmagicHDRMetadataEOTF::SDR:
							MediaPlayer->BlackmagicColorSettings->SetEncodingOverride(UE::Color::EEncoding::sRGB); // Missing support for BT.1886 so we default to sRGB instead.
							break;
						case EBlackmagicHDRMetadataEOTF::PQ:
							MediaPlayer->BlackmagicColorSettings->SetEncodingOverride(UE::Color::EEncoding::ST2084);
							break;
						case EBlackmagicHDRMetadataEOTF::HLG:
							MediaPlayer->BlackmagicColorSettings->SetEncodingOverride(UE::Color::EEncoding::HLG);
							break;
						default:
							checkNoEntry();
						}
						// User left encoding at None: reset any leftover reference white so the
						// plugin default (BT.2408 fallback in the sample for HDR) applies cleanly.
						MediaPlayer->BlackmagicColorSettings->SetReferenceWhiteOverride(UE::Color::EReferenceWhite::None);
					}

					if (MediaPlayer->SourceColorSettings.IsValid() && MediaPlayer->SourceColorSettings->HasColorSpaceOverride())
					{
						MediaPlayer->BlackmagicColorSettings->SetColorSpaceOverride(MediaPlayer->SourceColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB()));
					}
					else
					{
						switch (HDROptions.Gamut)
						{
						case EBlackmagicHDRMetadataGamut::Rec709:
							MediaPlayer->BlackmagicColorSettings->SetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB());
							break;
						case EBlackmagicHDRMetadataGamut::Rec2020:
							MediaPlayer->BlackmagicColorSettings->SetColorSpaceOverride(UE::Color::FColorSpace::GetRec2020());
							break;
						default:
							checkNoEntry();
						}
					}

					if (bIsProgressivePicture)
					{
						if (bEncodeTimecodeInTexel && DecodedTimecode.IsSet())
						{
							FTimecode SetTimecode = DecodedTimecode.GetValue();
							FMediaIOCoreEncodeTime EncodeTime(EncodePixelFormat, InFrameInfo.VideoBuffer, InFrameInfo.VideoPitch, InFrameInfo.VideoWidth, InFrameInfo.VideoHeight);
							EncodeTime.Render(SetTimecode.Hours, SetTimecode.Minutes, SetTimecode.Seconds, SetTimecode.Frames);
						}

						bool bGPUDirectTexturesAvailable = false;
						if (MediaPlayer->CanUseGPUTextureTransfer())
						{
							bGPUDirectTexturesAvailable = MediaPlayer->HasTextureAvailableForGPUTransfer();
							if (!bGPUDirectTexturesAvailable)
							{
								UE_LOGF(LogBlackmagicMedia, Error, "No texture available while doing a gpu texture transfer.");
							}
						}

						auto TextureSample = MediaPlayer->TextureSamplePool->AcquireShared();
						TextureSample->SetColorConversionSettings(MediaPlayer->OCIOSettings);
						bool bInitializeResult = false;
						if (bGPUDirectTexturesAvailable)
						{
							bInitializeResult = TextureSample->SetProperties(InFrameInfo.VideoPitch, InFrameInfo.VideoWidth, InFrameInfo.VideoHeight, SampleFormat, DecodedTime, MediaPlayer->VideoFrameRate, DecodedTimecode, MediaPlayer->BlackmagicColorSettings);
							if (bInitializeResult)
							{
								TextureSample->SetBuffer(InFrameInfo.VideoBuffer);
							}
						}
						else
						{
							bInitializeResult = TextureSample->Initialize(InFrameInfo.VideoBuffer
								, InFrameInfo.VideoPitch * InFrameInfo.VideoHeight
								, InFrameInfo.VideoPitch
								, InFrameInfo.VideoWidth
								, InFrameInfo.VideoHeight
								, SampleFormat
								, DecodedTime
								, MediaPlayer->VideoFrameRate
								, DecodedTimecode
								, MediaPlayer->BlackmagicColorSettings);
						}

						if (bInitializeResult)
						{
							if (bGPUDirectTexturesAvailable && MediaPlayer->CanUseGPUTextureTransfer())
							{
								// Mark this sample ready for GPU transfer
								TextureSample->SetAwaitingForGPUTransfer();

								// Keep the internal buffer alive until GPU transfer is finished
								OutBufferHolders.VideoBufferHolder = &TextureSample->GetBlackmagicInternalBufferLocker();
							}

							MediaPlayer->AddVideoSample(TextureSample);
						}
					}
					else
					{
						UE::MediaIOCore::FVideoFrame FrameInfo
						{
							InFrameInfo.VideoBuffer,
							(uint32)InFrameInfo.VideoPitch * InFrameInfo.VideoHeight,
							(uint32)InFrameInfo.VideoPitch,
							(uint32)InFrameInfo.VideoWidth,
							(uint32)InFrameInfo.VideoHeight,
							SampleFormat,
							DecodedTime,
							MediaPlayer->VideoFrameRate,
							DecodedTimecode,
							MediaPlayer->BlackmagicColorSettings
						};

						const TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> DeinterlacedSamples = MediaPlayer->Deinterlace(FrameInfo);

						for (const TSharedRef<FMediaIOCoreTextureSampleBase>& TextureSample : DeinterlacedSamples)
						{
							TextureSample->SetColorConversionSettings(MediaPlayer->OCIOSettings);
							MediaPlayer->AddVideoSample(TextureSample);
						}
					}
				}
			}
		}

		virtual void OnFrameFormatChanged(const BlackmagicDesign::FFormatInfo& NewFormat) override
		{
			if (MediaPlayer->bAutoDetect)
			{
				// NewFormat carries the frame rate (30/1 for 1080i60). FMediaIOMode stores the field
				// rate for interlaced standards (see FBlackmagicDeviceProvider::ToMediaMode), so double
				// the numerator to keep the auto-detected path consistent with manual mode selection.
				uint32 EffectiveNumerator = NewFormat.FrameRateNumerator;
				if (NewFormat.FieldDominance == BlackmagicDesign::EFieldDominance::Interlaced)
				{
					EffectiveNumerator *= 2;
				}

				MediaPlayer->VideoFrameRate = FFrameRate(EffectiveNumerator, NewFormat.FrameRateDenominator);
				MediaPlayer->BaseSettings.FrameRate = MediaPlayer->VideoFrameRate;

				MediaPlayer->VideoTrackFormat.Dim = FIntPoint(NewFormat.Width, NewFormat.Height);
				MediaPlayer->VideoTrackFormat.FrameRate = MediaPlayer->VideoFrameRate.AsDecimal();
				MediaPlayer->VideoTrackFormat.FrameRates = TRange<float>(MediaPlayer->VideoTrackFormat.FrameRate);

				MediaPlayer->SetupSampleChannels();

				MediaPlayer->NotifyVideoFormatDetected();

#if WITH_EDITOR
				MediaPlayer->PublishAutoDetectedFormatToSource(NewFormat);
#endif
			}
			else
			{
				UE_LOGF(LogBlackmagicMedia, Error, "The video format changed for '%ls'.", MediaPlayer ? *MediaPlayer->GetUrl() : TEXT("<Invalid>"));
				MediaState = EMediaState::Error;
			}
		}

		virtual void OnInterlacedOddFieldEvent(int64 FrameNumber) override
		{
			
		}

	private:
		TAtomic<int32> RefCounter;

		BlackmagicDesign::FUniqueIdentifier BlackmagicIdendifier;
		BlackmagicDesign::FChannelInfo ChannelInfo;

		mutable FCriticalSection CallbackLock;
		FBlackmagicMediaPlayer* MediaPlayer;

		EMediaState MediaState;

		BlackmagicDesign::FTimecode PreviousTimecode;
		FTimespan PrevousTimespan;
		bool bEncodeTimecodeInTexel;

		/** Number of audio bits per sample, audio channels and sample rate. */
		uint32 LastBitsPerSample;
		uint32 LastNumChannels;
		uint32 LastSampleRate;

		/** Frame drop count from the previous tick to keep track of deltas */
		int32 PreviousAudioFrameDropCount;
		int32 PreviousVideoFrameDropCount;

		int32 MaxNumAudioFrameBuffer;
		int32 MaxNumVideoFrameBuffer;

		/** Has video frame detection */
		double LastHasFrameTime;
		bool bReceivedValidFrame;

		bool bIsTimecodeExpected;
		bool bHasWarnedMissingTimecode;
	};
}

/* FBlackmagicVideoPlayer structors
*****************************************************************************/

FBlackmagicMediaPlayer::FBlackmagicMediaPlayer(IMediaEventSink& InEventSink)
	: Super(InEventSink)
	, AudioSamplePool(MakeUnique<FBlackmagicMediaAudioSamplePool>())
	, TextureSamplePool(MakeUnique<FBlackmagicMediaTextureSamplePool>())
	, SupportedSampleTypes(EMediaIOSampleType::None)
	, BlackmagicColorSettings(MakeShared<FNativeMediaSourceColorSettings>())
{
}

FBlackmagicMediaPlayer::~FBlackmagicMediaPlayer()
{
	Close();
}

/* IMediaPlayer interface
*****************************************************************************/

void FBlackmagicMediaPlayer::Close()
{
	if (EventCallback)
	{
		EventCallback->Uninitialize();
		EventCallback = nullptr;
	}

	AudioSamplePool->Reset();
	TextureSamplePool->Reset();

	//Disable all our channels from the monitor
	Samples->EnableTimedDataChannels(this, EMediaIOSampleType::None);

#if WITH_EDITOR
	TWeakObjectPtr<UObject> WeakSourceObj = MediaSourceObject;
	AsyncTask(ENamedThreads::GameThread, [WeakSourceObj]()
	{
		if (UCaptureCardMediaSource* Source = Cast<UCaptureCardMediaSource>(WeakSourceObj.Get()))
		{
			Source->ClearLastDetectedConfiguration();
		}
	});
#endif

	Super::Close();
}

FGuid FBlackmagicMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0x62a47ff5, 0xf61243a1, 0x9b377536, 0xc906c883);
	return PlayerPluginGUID;
}

/**
 * @EventName MediaFramework.BlackmagicMediaSourceOpened
 * @Trigger Triggered when a Blackmagic media source is opened through a media player.
 * @Type Client
 * @Owner MediaIO Team
 */
bool FBlackmagicMediaPlayer::Open(const FString& Url, const IMediaOptions* Options)
{
	if (!IBlackmagicMediaModule::Get().IsInitialized())
	{
		UE_LOGF(LogBlackmagicMedia, Error, "The BlackmagicMediaPlayer can't open URL '%ls'. Blackmagic is not initialized on your machine.", *Url);
		return false;
	}

	if (!Super::Open(Url, Options))
	{
		return false;
	}
	
	const EMediaIOAutoDetectableTimecodeFormat TimecodeFormat = (EMediaIOAutoDetectableTimecodeFormat)(Options->GetMediaOption(BlackmagicMediaOption::TimecodeFormat, (int64)EMediaIOAutoDetectableTimecodeFormat::None));
	const bool bAutoDetectTimecode = TimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::Auto;
	
	BlackmagicDesign::FChannelInfo ChannelInfo;
	ChannelInfo.DeviceIndex = Options->GetMediaOption(BlackmagicMediaOption::DeviceIndex, (int64)0);

	check(EventCallback == nullptr);
	EventCallback = new BlackmagicMediaPlayerHelpers::FBlackmagicMediaPlayerEventCallback(this, ChannelInfo);

	BlackmagicDesign::FInputChannelOptions ChannelOptions;
	ChannelOptions.bAutoDetect = bAutoDetect;
	ChannelOptions.CallbackPriority = 10;
	ChannelOptions.bReadVideo = Options->GetMediaOption(BlackmagicMediaOption::CaptureVideo, true);
	ChannelOptions.FormatInfo.DisplayMode = Options->GetMediaOption(BlackmagicMediaOption::BlackmagicVideoFormat, (int64)BlackmagicMediaOption::DefaultVideoFormat);
	BlackmagicColorFormat = (EBlackmagicMediaSourceColorFormat)(Options->GetMediaOption(BlackmagicMediaOption::ColorFormat, (int64)EBlackmagicMediaSourceColorFormat::YUV8));

	ChannelOptions.PixelFormat = BlackmagicColorFormat == EBlackmagicMediaSourceColorFormat::YUV8 ? BlackmagicDesign::EPixelFormat::pf_8Bits : BlackmagicDesign::EPixelFormat::pf_10Bits;
	
	switch (TimecodeFormat)
	{
	case EMediaIOAutoDetectableTimecodeFormat::Auto:
		ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_Auto;
		break;
	case EMediaIOAutoDetectableTimecodeFormat::LTC:
		ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_LTC;
		break;
	case EMediaIOAutoDetectableTimecodeFormat::VITC:
		ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_VITC1;
		break;
	case EMediaIOAutoDetectableTimecodeFormat::None:
	default:
		ChannelOptions.TimecodeFormat = BlackmagicDesign::ETimecodeFormat::TCF_None;
		break;
	}

	//Audio options
	{
		ChannelOptions.bReadAudio = Options->GetMediaOption(BlackmagicMediaOption::CaptureAudio, false);
		const EBlackmagicMediaAudioChannel AudioChannelOption = (EBlackmagicMediaAudioChannel)(Options->GetMediaOption(BlackmagicMediaOption::AudioChannelOption, (int64)EBlackmagicMediaAudioChannel::Stereo2));
		ChannelOptions.NumberOfAudioChannel = (AudioChannelOption == EBlackmagicMediaAudioChannel::Surround8) ? 8 : 2;
	}

	//Adjust supported sample types based on what's being captured
	SupportedSampleTypes = ChannelOptions.bReadVideo ? EMediaIOSampleType::Video : EMediaIOSampleType::None;
	SupportedSampleTypes |= ChannelOptions.bReadAudio ? EMediaIOSampleType::Audio : EMediaIOSampleType::None;
	Samples->EnableTimedDataChannels(this, SupportedSampleTypes);

	bVerifyFrameDropCount = Options->GetMediaOption(BlackmagicMediaOption::LogDropFrame, false);
	const bool bEncodeTimecodeInTexel = TimecodeFormat != EMediaIOAutoDetectableTimecodeFormat::None && Options->GetMediaOption(BlackmagicMediaOption::EncodeTimecodeInTexel, false);
	MaxNumAudioFrameBuffer = Options->GetMediaOption(BlackmagicMediaOption::MaxAudioFrameBuffer, (int64)8);
	MaxNumVideoFrameBuffer = Options->GetMediaOption(BlackmagicMediaOption::MaxVideoFrameBuffer, (int64)8);

	// Setup our different supported channels based on source settings
	SetupSampleChannels();

	const bool bSuccess = EventCallback->Initialize(ChannelOptions, bEncodeTimecodeInTexel, MaxNumAudioFrameBuffer, MaxNumVideoFrameBuffer);

	if (!bSuccess)
	{
		EventCallback->Uninitialize();
		EventCallback = nullptr;
	}

	return bSuccess;
}

void FBlackmagicMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan Timecode)
{
	// update player state
	EMediaState NewState = EventCallback ? EventCallback->GetMediaState() : EMediaState::Closed;

	if (NewState != CurrentState)
	{
		CurrentState = NewState;
		if (CurrentState == EMediaState::Playing)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
		}
		else if (NewState == EMediaState::Error)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
			Close();
		}
	}

	if (CurrentState != EMediaState::Playing)
	{
		return;
	}

	TickTimeManagement();
}

void FBlackmagicMediaPlayer::TickFetch(FTimespan DeltaTime, FTimespan Timecode)
{
	Super::TickFetch(DeltaTime, Timecode);
	if (IsHardwareReady())
	{
		ProcessFrame();
		VerifyFrameDropCount();
	}
}

#if WITH_EDITOR
const FSlateBrush* FBlackmagicMediaPlayer::GetDisplayIcon() const
{
	return IBlackmagicMediaModule::Get().GetStyle()->GetBrush("BlackmagicMediaIcon");
}
#endif //WITH_EDITOR

void FBlackmagicMediaPlayer::ProcessFrame()
{
	EventCallback->UpdateAudioTrackFormat(AudioTrackFormat);
}

void FBlackmagicMediaPlayer::VerifyFrameDropCount()
{
	EventCallback->VerifyFrameDropCount_GameThread(OpenUrl);
}

bool FBlackmagicMediaPlayer::IsHardwareReady() const
{
	return EventCallback && EventCallback->GetMediaState() == EMediaState::Playing;
}

void FBlackmagicMediaPlayer::SetupSampleChannels()
{
	FMediaIOSamplingSettings VideoSettings = BaseSettings;
	VideoSettings.BufferSize = MaxNumVideoFrameBuffer;
	Samples->InitializeVideoBuffer(VideoSettings);

	FMediaIOSamplingSettings AudioSettings = BaseSettings;
	AudioSettings.BufferSize = MaxNumAudioFrameBuffer;
	Samples->InitializeAudioBuffer(AudioSettings);
}

void FBlackmagicMediaPlayer::AddVideoSampleAfterGPUTransfer_RenderThread(const TSharedRef<FMediaIOCoreTextureSampleBase>& InSample)
{
	checkSlow(IsInRenderingThread());

	const TSharedRef<FBlackmagicMediaTextureSample> BlackMagicSample =
		StaticCastSharedRef<FBlackmagicMediaTextureSample, FMediaIOCoreTextureSampleBase>(InSample);

	// From now, the internal video buffer can be released
	BlackMagicSample->ReleaseBlackmagicInternalBuffer();

	// Let the base class do the rest
	Super::AddVideoSampleAfterGPUTransfer_RenderThread(InSample);
}

#if WITH_EDITOR
void FBlackmagicMediaPlayer::PublishAutoDetectedFormatToSource(const BlackmagicDesign::FFormatInfo& NewFormat)
{
	TWeakObjectPtr<UObject> WeakSourceObj = MediaSourceObject;
	BlackmagicDesign::FFormatInfo Snapshot = NewFormat;
	AsyncTask(ENamedThreads::GameThread, [WeakSourceObj, Snapshot]()
	{
		UBlackmagicMediaSource* Source = Cast<UBlackmagicMediaSource>(WeakSourceObj.Get());
		if (!Source)
		{
			return;
		}

		// Start from the configured connection (Device, Port, TransportType) and only overwrite the
		// mode fields the SDK actually detects (Resolution, FrameRate, Standard). Other FMediaIOMode
		// fields, notably DeviceModeIdentifier, are inherited from the asset and may be stale.
		FMediaIOConfiguration Detected = Source->MediaConfiguration;
		Detected.MediaMode.Resolution = FIntPoint(Snapshot.Width, Snapshot.Height);
		Detected.MediaMode.FrameRate = FFrameRate(Snapshot.FrameRateNumerator, Snapshot.FrameRateDenominator);
		switch (Snapshot.FieldDominance)
		{
		case BlackmagicDesign::EFieldDominance::Progressive:
			Detected.MediaMode.Standard = EMediaIOStandardType::Progressive;
			break;
		case BlackmagicDesign::EFieldDominance::Interlaced:
			Detected.MediaMode.Standard = EMediaIOStandardType::Interlaced;
			// Snapshot carries the frame rate (30/1 for 1080i60). FMediaIOMode stores the field rate
			// for interlaced standards (see FBlackmagicDeviceProvider::ToMediaMode), so double the
			// numerator to keep the auto-detected path consistent with manual mode selection.
			Detected.MediaMode.FrameRate.Numerator *= 2;
			break;
		case BlackmagicDesign::EFieldDominance::ProgressiveSegmentedFrame:
			Detected.MediaMode.Standard = EMediaIOStandardType::ProgressiveSegmentedFrame;
			break;
		default:
			checkNoEntry();
			break;
		}

		Source->SetLastDetectedConfiguration(Detected);
	});
}
#endif //WITH_EDITOR

#if WITH_EDITOR
FString FBlackmagicMediaPlayer::GetAnalyticsEventPrefix() const
{
	return TEXT("MediaFramework.Blackmagic");
}

void FBlackmagicMediaPlayer::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	IMediaIOCoreDeviceProvider* DeviceProvider = IMediaIOCoreModule::Get().GetDeviceProvider(FBlackmagicDeviceProvider::GetProviderName());
	if (!DeviceProvider)
	{
		return;
	}
	
	UMediaSource* MediaSource = Cast<UMediaSource>(MediaSourceObject);
	if (!MediaSource)
	{
		return;
	}

	const int64 ResolutionWidth = MediaSource->GetMediaOption(FMediaIOCoreMediaOption::ResolutionWidth, (int64)1920);
	const int64 ResolutionHeight = MediaSource->GetMediaOption(FMediaIOCoreMediaOption::ResolutionHeight, (int64)1080);
	
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionWidth"), FString::Printf(TEXT("%lld"), ResolutionWidth)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionHeight"), FString::Printf(TEXT("%lld"), ResolutionHeight)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("FrameRate"), *VideoFrameRate.ToPrettyText().ToString()));
	
	const int64 DeviceIndex = MediaSource->GetMediaOption(BlackmagicMediaOption::DeviceIndex, (int64)0);
	if (FMediaIOConfiguration* Configuration = DeviceProvider->GetConfigurations().FindByPredicate([DeviceIndex](const FMediaIOConfiguration& InConfiguration)
	{
		return InConfiguration.MediaConnection.Device.DeviceIdentifier == DeviceIndex;
	}))
	{
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Device"), Configuration->MediaConnection.Device.DeviceName));
			
		const FString SourceStr = FString::Format(TEXT("{0} {1}"), {
			DeviceProvider->GetTransportName(Configuration->MediaConnection.TransportType, Configuration->MediaConnection.QuadTransportType).ToString(),
			Configuration->MediaConnection.PortIdentifier });
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Source"), SourceStr));
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Standard"), UEnum::GetValueAsString(Configuration->MediaMode.Standard)));
	}
	
	if (BlackmagicColorSettings->HasColorSpaceOverride())
	{
		const UE::Color::FColorSpace& ColorSpace = BlackmagicColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB());
		FVector2d Red, Green, Blue, White;
		ColorSpace.GetChromaticities(Red, Green, Blue, White);
		const FString ColorSpaceStr = FString::Format(TEXT("R:{0} G:{1} B:{2} W:{3}"), { Red.ToString(), Green.ToString(), Blue.ToString(), White.ToString() });
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ColorSpaceOverride"), ColorSpaceStr));
	}
	
	if (BlackmagicColorSettings->HasEncodingOverride())
	{
		UE::Color::EEncoding Encoding = BlackmagicColorSettings->GetEncodingOverride();
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Encoding"), UEnum::GetValueAsString((ETextureSourceEncoding)Encoding)));
	}

	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesOCIO"), OCIOSettings.IsValid() && OCIOSettings->IsValid() ? TEXT("Yes") : TEXT("No")));
}
#endif

#undef LOCTEXT_NAMESPACE

