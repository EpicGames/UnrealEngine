// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCorePlayerBase.h"

#include "BlackmagicMediaPrivate.h"

#include "BlackmagicMediaSource.h"
#include "HAL/CriticalSection.h"
#include "MediaIOCoreAudioSampleBase.h"
#include "MediaIOCoreTextureSampleBase.h"
#include "MediaObjectPool.h"
#include "MediaShaders.h"

class IMediaEventSink;

enum class EMediaTextureSampleFormat;
enum class EMediaIOSampleType;

namespace BlackmagicMediaPlayerHelpers
{
	class FBlackmagicMediaPlayerEventCallback;
}

class FBlackmagicMediaTextureSample : public FMediaIOCoreTextureSampleBase
{
public:
	virtual const FMatrix& GetYUVToRGBMatrix() const override
	{
		return MediaShaders::YuvToRgbRec709Scaled;
	}

	//~ Begin IMediaPoolable interface
public:
	virtual void ShutdownPoolable() override
	{
		// Normally it should be explicitly released after GPU transfer. Just a bit of safety.
		ReleaseBlackmagicInternalBuffer();

		FMediaIOCoreTextureSampleBase::ShutdownPoolable();
	}
	//~ End IMediaPoolable interface

public:

	/** Returns buffer guard so it can be initialized outside (for performance reason) */
	TSharedPtr<BlackmagicDesign::IInputEventCallback::FFrameBufferHolder>& GetBlackmagicInternalBufferLocker()
	{
		return VideoBufferGuard;
	}

	/** Unlock internal buffer referenced by this sample */
	void ReleaseBlackmagicInternalBuffer()
	{
		VideoBufferGuard.Reset();
	}

private:

	/** Buffer guard instance */
	TSharedPtr<BlackmagicDesign::IInputEventCallback::FFrameBufferHolder> VideoBufferGuard;
};

class FBlackmagicMediaAudioSamplePool : public TMediaObjectPool<FMediaIOCoreAudioSampleBase> { };
class FBlackmagicMediaTextureSamplePool : public TMediaObjectPool<FBlackmagicMediaTextureSample> { };

/**
 * Implements a media player for Blackmagic.
 *
 * The processing of metadata and video frames is delayed until the fetch stage
 * (TickFetch) in order to increase the window of opportunity for receiving
 * frames for the current render frame time code.
 *
 * Depending on whether the media source enables time code synchronization,
 * the player's current play time (CurrentTime) is derived either from the
 * time codes embedded in frames or from the Engine's global time code.
 */
class FBlackmagicMediaPlayer : public FMediaIOCorePlayerBase
{
private:
	using Super = FMediaIOCorePlayerBase;

public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	FBlackmagicMediaPlayer(IMediaEventSink& InEventSink);

	/** Virtual destructor. */
	virtual ~FBlackmagicMediaPlayer();

public:

	//~ IMediaPlayer interface

	virtual void Close() override;
	virtual FGuid GetPlayerPluginGUID() const override;

	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;

	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;

	//~ ITimedDataInput interface
#if WITH_EDITOR
	virtual const FSlateBrush* GetDisplayIcon() const override;
#endif

public:

	/** Process pending audio and video frames, and forward them to the sinks. */
	void ProcessFrame();

	/** Verify if we lost some frames since last Tick. */
	void VerifyFrameDropCount();

	/** Is Hardware initialized */
	virtual bool IsHardwareReady() const override;

protected:
	//~ Begin FMediaIOCorePlayerBase interface
	virtual void SetupSampleChannels() override;
	virtual uint32 GetNumVideoFrameBuffers() const override 
	{
		return MaxNumVideoFrameBuffer;
	}
	virtual EMediaIOCoreColorFormat GetColorFormat() const override
	{
		return BlackmagicColorFormat == EBlackmagicMediaSourceColorFormat::YUV8 ? EMediaIOCoreColorFormat::YUV8 : EMediaIOCoreColorFormat::YUV10;
	}
	virtual void AddVideoSampleAfterGPUTransfer_RenderThread(const TSharedRef<FMediaIOCoreTextureSampleBase>& InSample) override;
	virtual TSharedPtr<FMediaIOCoreTextureSampleBase> AcquireTextureSample_AnyThread() const override
	{
		return TextureSamplePool->AcquireShared();
	}
	
#if WITH_EDITOR
	virtual FString GetAnalyticsEventPrefix() const override;
	virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const override;
#endif
	//~ End FMediaIOCorePlayerBase interface

private:

	friend BlackmagicMediaPlayerHelpers::FBlackmagicMediaPlayerEventCallback;
	BlackmagicMediaPlayerHelpers::FBlackmagicMediaPlayerEventCallback* EventCallback = nullptr;

#if WITH_EDITOR
	/**
	 * Surface the SDK-detected format on the source asset so the Media Profile editor can display
	 * the actual signal format instead of the stored default. Safe to call from any thread; the
	 * UObject mutation is dispatched onto the game thread.
	 */
	void PublishAutoDetectedFormatToSource(const BlackmagicDesign::FFormatInfo& NewFormat);
#endif

	/** Audio, MetaData, Texture  sample object pool. */
	TUniquePtr<FBlackmagicMediaAudioSamplePool> AudioSamplePool;
	TUniquePtr<FBlackmagicMediaTextureSamplePool> TextureSamplePool;

	/** Log warning about the amount of audio/video frame can't could not be cached . */
	bool bVerifyFrameDropCount = false;

	/** Max sample count our different buffer can hold. Taken from MediaSource */
	int32 MaxNumAudioFrameBuffer = 0;
	int32 MaxNumVideoFrameBuffer = 0;

	/** Used to flag which sample types we advertise as supported for timed data monitoring */
	EMediaIOSampleType SupportedSampleTypes;

	/** Blackmagic media source color format. */
	EBlackmagicMediaSourceColorFormat BlackmagicColorFormat = EBlackmagicMediaSourceColorFormat::YUV8;

	/** Container for local media source color setting overrides. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> BlackmagicColorSettings;
};
