// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef BLACKMAGIC_PLATFORM_TYPES_GUARD
	#undef BLACKMAGIC_PLATFORM_TYPES_GUARD
#else
	#error Mismatched BlackmagicHidePLatformTypes.h detected.
#endif

#undef DWORD
#undef FLOAT

#ifdef TRUE
	#undef TRUE
#endif

#ifdef FALSE
	#undef FALSE
#endif
