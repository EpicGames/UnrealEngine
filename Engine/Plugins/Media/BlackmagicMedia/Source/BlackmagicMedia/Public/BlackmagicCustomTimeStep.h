// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedCustomTimeStep.h"
#include "MediaIOCoreDefinitions.h"

#include "BlackmagicCustomTimeStep.generated.h"

namespace BlackmagicCustomTimeStepHelpers
{
	class FInputEventCallback;
	class FOutputEventCallback;
}

/**
 * Control the Engine TimeStep via the Blackmagic Design card.
 */
UCLASS(Blueprintable, editinlinenew, meta=(DisplayName="Blackmagic SDI Input", MediaIOCustomLayout="Blackmagic"))
class BLACKMAGICMEDIA_API UBlackmagicCustomTimeStep : public UGenlockedCustomTimeStep
{
	GENERATED_UCLASS_BODY()

public:
	//~ UEngineCustomTimeStep interface
	virtual bool Initialize(class UEngine* InEngine) override;
	virtual void Shutdown(class UEngine* InEngine) override;
	virtual bool UpdateTimeStep(class UEngine* InEngine) override;
	virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override;
	virtual FFrameRate GetFixedFrameRate() const override;

	//~ UGenlockedCustomTimeStep interface
	virtual uint32 GetLastSyncCountDelta() const override;
	virtual bool IsLastSyncDataValid() const override;
	virtual FFrameRate GetSyncRate() const override;
	virtual bool WaitForSync() override;
	virtual bool SupportsFormatAutoDetection() const { return true; };

	//~ UObject interface
	virtual void BeginDestroy() override;

private:
	void ReleaseResources();

public:
	/** The device, port and video settings that correspond to where the Genlock signal will be coming from */
	UPROPERTY(EditAnywhere, Category="Genlock", meta=(DisplayName="Configuration"))
	FMediaIOConfiguration MediaConfiguration;

	/** Enable mechanism to detect Engine loop overrunning the source */
	UPROPERTY(EditAnywhere, Category="Genlock options", meta=(DisplayName="Display Dropped Frames Warning"))
	bool bEnableOverrunDetection;

private:
	friend BlackmagicCustomTimeStepHelpers::FInputEventCallback;
	BlackmagicCustomTimeStepHelpers::FInputEventCallback* InputEventCallback;

	bool bWarnedAboutVSync;
};
