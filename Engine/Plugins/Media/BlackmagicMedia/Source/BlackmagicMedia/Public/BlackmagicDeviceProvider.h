// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "UObject/ObjectMacros.h"

#include "BlackmagicDeviceProvider.generated.h"

/**
 * Available timecode formats that Blackmagic support.
 */
UENUM()
enum class EBlackmagicMediaTimecodeFormat : uint8
{
	None,
	LTC,
	VITC,
};

/**
 * Implementation of IMediaIOCoreDeviceProvider for Blackmagic
 */
class BLACKMAGICMEDIA_API FBlackmagicDeviceProvider : public IMediaIOCoreDeviceProvider
{
public:
	static FName GetProviderName();
	static FName GetProtocolName();

public:
	virtual FName GetFName() override;

	virtual TArray<FMediaIOConnection> GetConnections() const override;
	virtual TArray<FMediaIOConfiguration> GetConfigurations() const override;
	virtual TArray<FMediaIOConfiguration> GetConfigurations(bool bAllowInput, bool bAllowOutput) const override;
	virtual TArray<FMediaIODevice> GetDevices() const override;
	virtual TArray<FMediaIOMode> GetModes(const FMediaIODevice& InDevice, bool bInOutput) const override;
	virtual TArray<FMediaIOInputConfiguration> GetInputConfigurations() const override;
	virtual TArray<FMediaIOOutputConfiguration> GetOutputConfigurations() const override;
	virtual TArray<FMediaIOVideoTimecodeConfiguration> GetTimecodeConfigurations() const override;

	virtual FMediaIOConfiguration GetDefaultConfiguration() const override;
	virtual FMediaIOMode GetDefaultMode() const override;
	virtual FMediaIOInputConfiguration GetDefaultInputConfiguration() const override;
	virtual FMediaIOOutputConfiguration GetDefaultOutputConfiguration() const override;
	virtual FMediaIOVideoTimecodeConfiguration GetDefaultTimecodeConfiguration() const override;

	virtual UMediaSource* CreateMediaSource(const FMediaIOConfiguration& InConfiguration,
		UObject* Outer) const override;

	virtual FText ToText(const FMediaIOConfiguration& InConfiguration, bool bInIsAutoDetected = false) const override;
	virtual FText ToText(const FMediaIOConnection& InConnection) const override;
	virtual FText ToText(const FMediaIOOutputConfiguration& InConfiguration) const override;

#if WITH_EDITOR
	virtual bool ShowInputTransportInSelector() const override { return false; }
	virtual bool ShowInputKeyInSelector() const override { return false; }
	virtual bool ShowOutputKeyInSelector() const override { return false; }
#endif
};
