// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "BlackmagicMediaDefinitions.generated.h"

namespace BlackmagicDesign
{
	struct FHDRMetaData;
}

/**
 * HDR Transfer function.
 * Must match Blackmagic::EBlackmagicHDRMetadataEOTF.
 */
UENUM()
enum class EBlackmagicHDRMetadataEOTF : uint8
{
	SDR,
	HDR,
	PQ,
	HLG,
};
	
/**
 * HDR Color Gamut.
 * Must match Blackmagic::EBlackmagicHDRMetadataGamut.
 */
UENUM()
enum class EBlackmagicHDRMetadataGamut : uint8
{
	Rec709,
	Rec2020,
};

/**
 * Set of metadata describing a HDR video signal.
 */
USTRUCT()
struct BLACKMAGICMEDIA_API FBlackmagicMediaHDROptions
{
	GENERATED_BODY()

	/** Transfer function to use for converting the video signal to an optical signal. */
	UPROPERTY(EditAnywhere, Category = "HDR")
	EBlackmagicHDRMetadataEOTF EOTF = EBlackmagicHDRMetadataEOTF::SDR;

	/** The color gamut of the video signal. */
	UPROPERTY(EditAnywhere, Category = "HDR")
	EBlackmagicHDRMetadataGamut Gamut = EBlackmagicHDRMetadataGamut::Rec709;
};

namespace UE::BlackmagicMedia
{
	/** Create Blackmagic HDR UStruct from the BlackmagicLib's hdr struct. */
	FBlackmagicMediaHDROptions MakeBlackmagicMediaHDROptions(const BlackmagicDesign::FHDRMetaData& HDROptions);
	/** Create BlackmagicLib's hdr struct from the Blackmagic HDR UStruct. */
	BlackmagicDesign::FHDRMetaData MakeBlackmagicHDROptions(const FBlackmagicMediaHDROptions& HDROptions);
}
