// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureCardMediaSource.h"

#include "BlackmagicDeviceProvider.h"
#include "MediaIOCoreDefinitions.h"

#include "BlackmagicMediaSource.generated.h"


/**
 * Native data format.
 */
UENUM()
enum class EBlackmagicMediaSourceColorFormat : uint8
{
	YUV8 UMETA(DisplayName = "8bit YUV"),
	YUV10 UMETA(DisplayName = "10bit YUV"),
};

/**
 * Available number of audio channel supported by Unreal Engine & Capture card.
 */
UENUM(BlueprintType)
enum class EBlackmagicMediaAudioChannel : uint8
{
	Stereo2,
	Surround8,
};

/**
 * Media source description for Blackmagic.
 */
UCLASS(BlueprintType, hideCategories=(Platforms,Object), PrioritizeCategories=(Blackmagic, Video, ColorManagement, Audio, Ancillary, Synchronization, Debug), meta=(MediaIOCustomLayout="Blackmagic"))
class BLACKMAGICMEDIA_API UBlackmagicMediaSource : public UCaptureCardMediaSource
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	UBlackmagicMediaSource();

public:

	/** The device, port and video settings that correspond to the input. */
	UPROPERTY(EditAnywhere, Category="Blackmagic", meta=(DisplayName="Configuration"))
	FMediaIOConfiguration MediaConfiguration;
	
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "Use AutoDetectableTimecodeFormat")
	/** Use the time code embedded in the input stream. */
	UPROPERTY(meta=(DeprecatedProperty))
	EMediaIOTimecodeFormat TimecodeFormat_DEPRECATED;
#endif

	/** Use the time code embedded in the input stream. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Blackmagic")
	EMediaIOAutoDetectableTimecodeFormat AutoDetectableTimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::Auto;
public:
	/** Capture Audio from the Blackmagic source. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Audio")
	bool bCaptureAudio;

	/** Desired number of audio channel to capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Audio, meta=(EditCondition="bCaptureAudio"))
	EBlackmagicMediaAudioChannel AudioChannels;

	/** Maximum number of audio frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Audio", meta=(EditCondition="bCaptureAudio", ClampMin="1", ClampMax="32"))
	int32 MaxNumAudioFrameBuffer;

public:
	/** Capture Video from the Blackmagic source. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Video")
	bool bCaptureVideo;

	/** Native data format internally used by the device after being converted from SDI/HDMI signal. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Video", meta=(EditCondition="bCaptureVideo"))
	EBlackmagicMediaSourceColorFormat ColorFormat;

#if WITH_EDITORONLY_DATA
	/**
	 * Whether the video input is in sRGB color space.
	 * A sRGB to Linear conversion will be applied resulting in a texture in linear space.
	 * @Note If the texture is not in linear space, it won't look correct in the editor. Another pass will be required either through Composure or other means.
	 */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use ColorEncoding instead."))
	bool bIsSRGBInput_DEPRECATED = false;
#endif

	/** Maximum number of video frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Video", meta=(EditCondition="bCaptureVideo", ClampMin="1", ClampMax="32"))
	int32 MaxNumVideoFrameBuffer;

public:
	/** Log a warning when there's a drop frame. */
	UPROPERTY(EditAnywhere, Category="Debug")
	bool bLogDropFrame;

	/**
	 * Burn Frame Timecode in the input texture without any frame number clipping.
	 * @Note Only supported in progressive format.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Debug", meta = (DisplayName = "Burn Frame Timecode"))
	bool bEncodeTimecodeInTexel;

public:
	//~ UTimeSynchronizableMediaSource Interface
	virtual bool SupportsFormatAutoDetection() const { return true; }

	//~ IMediaOptions interface

	virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	virtual bool HasMediaOption(const FName& Key) const override;

public:
	//~ UMediaSource interface

	virtual FString GetUrl() const override;
	virtual bool Validate() const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
	/** Assigns a default configuration (The first one in the list of configurations). */
	void AssignDefaultConfiguration();

public:
	//~ UObject interface
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InPropertyChangedEvent) override;
#endif //WITH_EDITOR
	virtual void PostLoad() override;
	//~ End UObject interface
};
