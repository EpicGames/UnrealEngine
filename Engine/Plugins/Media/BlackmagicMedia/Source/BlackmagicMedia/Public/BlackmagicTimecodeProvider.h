// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedTimecodeProvider.h"
#include "MediaIOCoreDefinitions.h"

#include "BlackmagicTimecodeProvider.generated.h"

namespace BlackmagicTimecodeProviderHelpers
{
	class FEventCallback;
	class FEventCallbackWrapper;
}

/**
 * Class to fetch a timecode via a Blackmagic Design card.
 */
UCLASS(Blueprintable, editinlinenew, meta=(DisplayName="Blackmagic SDI Input", MediaIOCustomLayout="Blackmagic"))
class BLACKMAGICMEDIA_API UBlackmagicTimecodeProvider : public UGenlockedTimecodeProvider
{
	GENERATED_BODY()

public:
	UBlackmagicTimecodeProvider();

	//~ UTimecodeProvider interface
	virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) override;
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override;
	virtual bool Initialize(class UEngine* InEngine) override;
	virtual void Shutdown(class UEngine* InEngine) override;
	/** Whether this provider supports format autodetection. */

	virtual bool SupportsAutoDetected() const override
	{
		return true;
	}
    
	virtual void SetIsAutoDetected(bool bInIsAutoDetected) override
	{
		bAutoDetectTimecode = bInIsAutoDetected;
	}
	
    virtual bool IsAutoDetected() const override
	{
		return bAutoDetectTimecode;
	}
	
	//~ UObject interface
	virtual void BeginDestroy() override;
	virtual void PostLoad() override;

private:
	void ReleaseResources();

public:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "Use TimecodeConfiguration instead")
	/** The device, port and video settings that correspond to the input. */
	UPROPERTY(meta=(DeprecatedProperty))
	FMediaIOConfiguration MediaConfiguration_DEPRECATED;

	UE_DEPRECATED(5.1, "Use TimecodeConfiguration instead")
	/** Use the time code embedded in the input stream. */
	/** Timecode format to read from a video signal. */
	UPROPERTY(meta=(DeprecatedProperty))
	EMediaIOTimecodeFormat TimecodeFormat_DEPRECATED;
#endif
	
	/** Use the time code embedded in the input stream. */
    /** Timecode format to read from a video signal. */
	UPROPERTY(EditAnywhere, Category = "Blackmagic")
	FMediaIOVideoTimecodeConfiguration TimecodeConfiguration;
	
	UPROPERTY()
	bool bAutoDetectTimecode = true;

private:
	friend BlackmagicTimecodeProviderHelpers::FEventCallback;
	BlackmagicTimecodeProviderHelpers::FEventCallback* EventCallback;
};
