// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FSlateStyleSet;
class IMediaEventSink;
class IMediaPlayer;

/**
 * Interface for the Media module.
 */
class IBlackmagicMediaModule : public IModuleInterface
{
public:

	static inline IBlackmagicMediaModule& Get()
	{
		static const FName ModuleName = "BlackmagicMedia";
		return FModuleManager::LoadModuleChecked<IBlackmagicMediaModule>(ModuleName);
	}

	/**
	 * Create an Blackmagic based media player.
	 * @param EventSink The object that receives media events from the player.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) = 0;

	/** @return SlateStyleSet to be used across the BlackmagicMedia module */
	virtual TSharedPtr<FSlateStyleSet> GetStyle() = 0;

	/** @return true if the Blackmagic module and VideoIO.dll could be loaded */
	virtual bool IsInitialized() const = 0;

	/** @return true if the Blackmagic card can be used */
	virtual bool CanBeUsed() const = 0;
};

