// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class BlackmagicMediaEditor : ModuleRules
	{
		public BlackmagicMediaEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"EditorFramework",
					"MediaIOCore",
					"UnrealEd"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetDefinition",
					"BlackmagicMedia",
					"BlackmagicMediaOutput",
					"Engine",
					"MediaAssets",
					"MediaIOEditor",
					"MediaPlayerEditor",
					"Projects",
					"SlateCore",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AssetTools",
				});
		}
	}
}
