// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "BlackmagicMediaSource.h"
#include "AssetDefinition_BlackmagicMediaSource.generated.h"

UCLASS()
class UAssetDefinition_BlackmagicMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_BlackmagicMediaSource", "Blackmagic Media Source"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UBlackmagicMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_BlackmagicMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
