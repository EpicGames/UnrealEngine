// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Templates/UniquePtr.h"

#define LOCTEXT_NAMESPACE "BlackmagicMediaEditor"

/**
 * Implements the MediaEditor module.
 */
class FBlackmagicMediaEditorModule : public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		RegisterStyle();
	}

	virtual void ShutdownModule() override
	{
		if (!UObjectInitialized() && !IsEngineExitRequested())
		{
			UnregisterStyle();	
		}
	}

private:
	TUniquePtr<FSlateStyleSet> StyleInstance;

private:

	void RegisterStyle()
	{
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleInstance->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

		StyleInstance = MakeUnique<FSlateStyleSet>("BlackmagicStyle");

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("BlackmagicMedia"));
		if (Plugin.IsValid())
		{
			StyleInstance->SetContentRoot(FPaths::Combine(Plugin->GetContentDir(), TEXT("Editor/Icons")));
		}

		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		StyleInstance->Set("ClassThumbnail.BlackmagicMediaSource", new IMAGE_BRUSH("BlackmagicMediaSource_64x", Icon64x64));
		StyleInstance->Set("ClassIcon.BlackmagicMediaSource", new IMAGE_BRUSH("BlackmagicMediaSource_20x", Icon20x20));
		StyleInstance->Set("ClassThumbnail.BlackmagicMediaOutput", new IMAGE_BRUSH("BlackmagicMediaOutput_64x", Icon64x64));
		StyleInstance->Set("ClassIcon.BlackmagicMediaOutput", new IMAGE_BRUSH("BlackmagicMediaOutput_20x", Icon20x20));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance.Get());

#undef IMAGE_BRUSH
	}

	void UnregisterStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance.Get());
		StyleInstance.Reset();
	}
};



IMPLEMENT_MODULE(FBlackmagicMediaEditorModule, BlackmagicMediaEditor);

#undef LOCTEXT_NAMESPACE

