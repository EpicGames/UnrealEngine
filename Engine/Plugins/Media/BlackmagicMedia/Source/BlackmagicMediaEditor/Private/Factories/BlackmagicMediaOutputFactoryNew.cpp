// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicMediaOutputFactoryNew.h"

#include "AssetTypeCategories.h"
#include "BlackmagicMediaOutput.h"
#include "EngineAnalytics.h"


/* UBlackmagicMediaOutputFactoryNew structors
 *****************************************************************************/

UBlackmagicMediaOutputFactoryNew::UBlackmagicMediaOutputFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UBlackmagicMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

/**
 * @EventName MediaFramework.BlackmagicMediaOutputCreated
 * @Trigger Triggered when a Blackmagic media output asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* UBlackmagicMediaOutputFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.BlackmagicMediaOutputCreated"));
	}
	return NewObject<UBlackmagicMediaOutput>(InParent, InClass, InName, Flags);
}


uint32 UBlackmagicMediaOutputFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UBlackmagicMediaOutputFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
