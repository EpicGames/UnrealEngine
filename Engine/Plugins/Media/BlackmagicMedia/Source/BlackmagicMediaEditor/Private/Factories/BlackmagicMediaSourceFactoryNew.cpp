// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicMediaSourceFactoryNew.h"

#include "AssetTypeCategories.h"
#include "BlackmagicMediaSource.h"


/* UBlackmagicMediaSourceFactoryNew structors
 *****************************************************************************/

UBlackmagicMediaSourceFactoryNew::UBlackmagicMediaSourceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UBlackmagicMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UBlackmagicMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UBlackmagicMediaSource* Source = NewObject<UBlackmagicMediaSource>(InParent, InClass, InName, Flags);
	Source->AssignDefaultConfiguration();
	return Source;
}


uint32 UBlackmagicMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UBlackmagicMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
