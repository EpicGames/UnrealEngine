// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "IBlackmagicMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "IMediaModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


#define LOCTEXT_NAMESPACE "BlackmagicMediaFactoryModule"


/**
 * Implements the MediaFactory module.
 */
class FBlackmagicMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:
	static const FName NAME_Protocol;

	//~ IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		return 100;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto MediaModule = FModuleManager::LoadModulePtr<IBlackmagicMediaModule>("BlackmagicMedia");
		return (MediaModule != nullptr) ? MediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Blackmagic Device Interface");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("BlackmagicMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x62a47ff5, 0xf61243a1, 0x9b377536, 0xc906c883);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return Feature == EMediaFeature::AudioSamples
			|| Feature == EMediaFeature::VideoSamples;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported platforms
		SupportedPlatforms.Add(TEXT("Windows"));
		SupportedPlatforms.Add(TEXT("Linux"));

		// supported schemes
		SupportedUriSchemes.Add(NAME_Protocol.ToString());

		// register player factory
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:
	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};

const FName FBlackmagicMediaFactoryModule::NAME_Protocol = "blackmagicdesign"; // also defined in FBlackmagicDeviceProvider

IMPLEMENT_MODULE(FBlackmagicMediaFactoryModule, BlackmagicMediaFactory);

#undef LOCTEXT_NAMESPACE
