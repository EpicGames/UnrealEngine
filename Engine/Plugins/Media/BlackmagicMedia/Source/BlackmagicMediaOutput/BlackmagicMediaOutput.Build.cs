// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class BlackmagicMediaOutput : ModuleRules
	{
		public BlackmagicMediaOutput(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"BlackmagicMedia",
					"MediaIOCore",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"BlackmagicCore",
					"Core",
					"CoreUObject",
					"Engine",
					"GPUTextureTransfer",
					"RenderCore",
					"RHI",
					"Slate"
				}
            );
		}
	}
}
