// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicMediaCapture.h"

#include "BlackmagicDeviceProvider.h"
#include "BlackmagicLib.h"
#include "BlackmagicMediaOutput.h"
#include "BlackmagicMediaOutputModule.h"
#include "ColorManagement/ColorSpace.h"
#include "GPUTextureTransfer.h"
#include "GPUTextureTransferModule.h"
#include "Engine/Engine.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "IBlackmagicMediaModule.h"
#include "IMediaIOCoreModule.h"
#include "MediaIOCoreSubsystem.h"
#include "MediaIOCoreFileWriter.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SViewport.h"

#if WITH_EDITOR
#include "AnalyticsEventAttribute.h"
#include "EngineAnalytics.h"
#endif

static FAutoConsoleVariableDeprecated CVarBlackmagicEnableGPUDirect_Deprecated(TEXT("Blackmagic.EnableGPUDirect"), TEXT("MediaIO.EnableGPUDirect"), TEXT("5.6"));

bool bBlackmagicWritInputRawDataCmdEnable = false;
static FAutoConsoleCommand BlackmagicWriteInputRawDataCmd(
	TEXT("Blackmagic.WriteInputRawData"),
	TEXT("Write Blackmagic raw input buffer to file."),
	FConsoleCommandDelegate::CreateLambda([]() { bBlackmagicWritInputRawDataCmdEnable = true; })
	);

namespace BlackmagicMediaCaptureHelpers
{
	BlackmagicDesign::FHDRMetaData MakeBlackmagicHDRMetadata(const FBlackmagicMediaHDROptions& HDROptions)
	{
		BlackmagicDesign::FHDRMetaData HDRMetadata;
		HDRMetadata.bIsAvailable = true;
		HDRMetadata.EOTF = (BlackmagicDesign::EHDRMetaDataEOTF) HDROptions.EOTF;

		UE::Color::EColorSpace ColorSpaceEnum = UE::Color::EColorSpace::sRGB;
		switch (HDROptions.Gamut)
		{
		case EBlackmagicHDRMetadataGamut::Rec709:
			HDRMetadata.ColorSpace = BlackmagicDesign::EHDRMetaDataColorspace::Rec709;
			ColorSpaceEnum = UE::Color::EColorSpace::sRGB;
			break;
		case EBlackmagicHDRMetadataGamut::Rec2020:
			HDRMetadata.ColorSpace = BlackmagicDesign::EHDRMetaDataColorspace::Rec2020;
			ColorSpaceEnum = UE::Color::EColorSpace::Rec2020;
			break;
		default:
			HDRMetadata.bIsAvailable = false;
			checkNoEntry();
			break;
		}
		
		UE::Color::FColorSpace ColorSpace(ColorSpaceEnum);
		HDRMetadata.WhitePointX = ColorSpace.GetWhiteChromaticity().X;
		HDRMetadata.WhitePointY = ColorSpace.GetWhiteChromaticity().Y;

		HDRMetadata.DisplayPrimariesRedX = ColorSpace.GetRedChromaticity().X;
		HDRMetadata.DisplayPrimariesRedY = ColorSpace.GetRedChromaticity().Y;

		HDRMetadata.DisplayPrimariesGreenX = ColorSpace.GetGreenChromaticity().X;
		HDRMetadata.DisplayPrimariesGreenY = ColorSpace.GetGreenChromaticity().Y;
		
		HDRMetadata.DisplayPrimariesBlueX = ColorSpace.GetBlueChromaticity().X;
		HDRMetadata.DisplayPrimariesBlueY = ColorSpace.GetBlueChromaticity().Y;

		return HDRMetadata;
	}
	
	struct BLACKMAGICCORE_API FHDRMetaData
	{
		FHDRMetaData();

		double WhitePointX;
		double WhitePointY;
		double DisplayPrimariesRedX;
		double DisplayPrimariesRedY;
		double DisplayPrimariesGreenX;
		double DisplayPrimariesGreenY;
		double DisplayPrimariesBlueX;
		double DisplayPrimariesBlueY;
		double MaxDisplayLuminance;
		double MinDisplayLuminance;
		double MaxContentLightLevel;
		double MaxFrameAverageLightLevel;
	};
	
	class FBlackmagicMediaCaptureEventCallback : public BlackmagicDesign::IOutputEventCallback
	{
	public:
		FBlackmagicMediaCaptureEventCallback(UBlackmagicMediaCapture* InOwner, const BlackmagicDesign::FChannelInfo& InChannelInfo)
			: RefCounter(0)
			, Owner(InOwner)
			, ChannelInfo(InChannelInfo)
			, LastFramesDroppedCount(0)
		{
		}

		bool Initialize(const BlackmagicDesign::FOutputChannelOptions& InChannelOptions)
		{
			AddRef();

			check(!BlackmagicIdendifier.IsValid());
			BlackmagicDesign::ReferencePtr<BlackmagicDesign::IOutputEventCallback> SelfCallbackRef(this);
			BlackmagicIdendifier = BlackmagicDesign::RegisterOutputChannel(ChannelInfo, InChannelOptions, SelfCallbackRef);
			return BlackmagicIdendifier.IsValid();
		}

		void Uninitialize()
		{
			{
				FScopeLock Lock(&CallbackLock);
				BlackmagicDesign::UnregisterOutputChannel(ChannelInfo, BlackmagicIdendifier, true);
				Owner = nullptr;
			}

			Release();
		}

		bool SendVideoFrameData(BlackmagicDesign::FFrameDescriptor& InFrameDescriptor)
		{
			return BlackmagicDesign::SendVideoFrameData(ChannelInfo, InFrameDescriptor);
		}

		bool SendVideoFrameData(BlackmagicDesign::FFrameDescriptor_GPUDMA& InFrameDescriptor)
		{
			return BlackmagicDesign::SendVideoFrameData(ChannelInfo, InFrameDescriptor);
		}

		bool SendAudioSamples(const BlackmagicDesign::FAudioSamplesDescriptor& InAudioDescriptor)
		{
			return BlackmagicDesign::SendAudioSamples(ChannelInfo, InAudioDescriptor);
		}

		bool ShouldCaptureThisFrame(FTimecode Timecode, uint32 FrameNumber) const
		{
			return BlackmagicDesign::ShouldCaptureThisFrame(ChannelInfo, Timecode, FrameNumber);
		}

	private:

		virtual void AddRef() override
		{
			++RefCounter;
		}

		virtual void Release() override
		{
			--RefCounter;
			if (RefCounter == 0)
			{
				delete this;
			}
		}

		virtual void OnInitializationCompleted(bool bSuccess)
		{
			FScopeLock Lock(&CallbackLock);
			if (Owner != nullptr)
			{
				Owner->SetState(bSuccess ? EMediaCaptureState::Capturing : EMediaCaptureState::Error);
			}
		}

		virtual void OnShutdownCompleted() override
		{
			FScopeLock Lock(&CallbackLock);
			if (Owner != nullptr)
			{
				Owner->SetState(EMediaCaptureState::Stopped);
				if (Owner->WakeUpEvent)
				{
					Owner->WakeUpEvent->Trigger();
				}
			}
		}


		virtual void OnOutputFrameCopied(const FFrameSentInfo& InFrameInfo)
		{
			FScopeLock Lock(&CallbackLock);
			if (Owner != nullptr)
			{
				if (Owner->WakeUpEvent)
				{
					Owner->WakeUpEvent->Trigger();
				}

				if (Owner->bLogDropFrame)
				{
					const uint32 FrameDropCount = InFrameInfo.FramesDropped;
					if (FrameDropCount > LastFramesDroppedCount)
					{
						UE_LOGF(LogBlackmagicMediaOutput, Warning, "Lost %d frames on Blackmagic device %d. Frame rate may be too slow.", FrameDropCount - LastFramesDroppedCount, ChannelInfo.DeviceIndex);
					}
					LastFramesDroppedCount = FrameDropCount;
				}
			}
		}

		virtual void OnPlaybackStopped()
		{
			FScopeLock Lock(&CallbackLock);
			if (Owner != nullptr)
			{
				Owner->SetState(EMediaCaptureState::Error);
				if (Owner->WakeUpEvent)
				{
					Owner->WakeUpEvent->Trigger();
				}
			}
		}

		virtual void OnInterlacedOddFieldEvent()
		{
			FScopeLock Lock(&CallbackLock);
			if (Owner != nullptr && Owner->WakeUpEvent)
			{
				Owner->WakeUpEvent->Trigger();
			}
		}


	private:
		TAtomic<int32> RefCounter;
		mutable FCriticalSection CallbackLock;
		UBlackmagicMediaCapture* Owner;

		BlackmagicDesign::FChannelInfo ChannelInfo;
		BlackmagicDesign::FUniqueIdentifier BlackmagicIdendifier;
		uint32 LastFramesDroppedCount;
	};

	BlackmagicDesign::EFieldDominance GetFieldDominanceFromMediaStandard(EMediaIOStandardType StandardType)
	{
		switch(StandardType)
		{
			case EMediaIOStandardType::Interlaced:
				return BlackmagicDesign::EFieldDominance::Interlaced;
			case EMediaIOStandardType::ProgressiveSegmentedFrame:
				return BlackmagicDesign::EFieldDominance::ProgressiveSegmentedFrame;
			case EMediaIOStandardType::Progressive:
			default:
				return BlackmagicDesign::EFieldDominance::Progressive;
		}
	}

	BlackmagicDesign::EPixelFormat ConvertPixelFormat(EBlackmagicMediaOutputPixelFormat PixelFormat)
	{
		switch (PixelFormat)
        {
	        case EBlackmagicMediaOutputPixelFormat::PF_8BIT_YUV:
        		return BlackmagicDesign::EPixelFormat::pf_8Bits;
	        case EBlackmagicMediaOutputPixelFormat::PF_10BIT_YUV:
	        default:
        		return BlackmagicDesign::EPixelFormat::pf_10Bits;
        }
	}
	
	BlackmagicDesign::ETimecodeFormat ConvertTimecodeFormat(EMediaIOTimecodeFormat TimecodeFormat)
	{
		switch (TimecodeFormat)
		{
			case EMediaIOTimecodeFormat::LTC:
				return BlackmagicDesign::ETimecodeFormat::TCF_LTC;
			case EMediaIOTimecodeFormat::VITC:
				return BlackmagicDesign::ETimecodeFormat::TCF_VITC1;
			case EMediaIOTimecodeFormat::None:
			default:
				return BlackmagicDesign::ETimecodeFormat::TCF_None;
		}
	}

	BlackmagicDesign::ELinkConfiguration ConvertTransportType(EMediaIOTransportType TransportType, EMediaIOQuadLinkTransportType QuadlinkTransportType)
	{
		switch (TransportType)
		{
		case EMediaIOTransportType::SingleLink:
		case EMediaIOTransportType::HDMI: // Blackmagic support HDMI but it is not shown in UE's UI. It's configured in BMD design tool and it's considered a normal link by UE.
			return BlackmagicDesign::ELinkConfiguration::SingleLink;
		case EMediaIOTransportType::DualLink:
			return BlackmagicDesign::ELinkConfiguration::DualLink;
		case EMediaIOTransportType::QuadLink:
		default:
			if (QuadlinkTransportType == EMediaIOQuadLinkTransportType::SquareDivision)
			{
				return BlackmagicDesign::ELinkConfiguration::QuadLinkSqr;
			}
			return BlackmagicDesign::ELinkConfiguration::QuadLinkTSI;
		}
	}

	BlackmagicDesign::EAudioBitDepth ConvertAudioBitDepth(EBlackmagicMediaOutputAudioBitDepth BitDepth)
	{
		switch(BitDepth)
		{
		case EBlackmagicMediaOutputAudioBitDepth::Signed_16Bits: return BlackmagicDesign::EAudioBitDepth::Signed_16Bits;
		case EBlackmagicMediaOutputAudioBitDepth::Signed_32Bits: return BlackmagicDesign::EAudioBitDepth::Signed_32Bits;
		default:
			checkNoEntry();
			return BlackmagicDesign::EAudioBitDepth::Signed_32Bits;
		}
	}
}

/* namespace BlackmagicMediaCaptureDevice
*****************************************************************************/
namespace BlackmagicMediaCaptureDevice
{
	BlackmagicDesign::FTimecode ConvertToBlackmagicTimecode(const FTimecode& InTimecode, float InEngineFPS, float InBlackmagicFPS)
	{
		const float Divider = InEngineFPS / InBlackmagicFPS;

		BlackmagicDesign::FTimecode Timecode;
		Timecode.Hours = InTimecode.Hours;
		Timecode.Minutes = InTimecode.Minutes;
		Timecode.Seconds = InTimecode.Seconds;
		Timecode.Frames = int32(float(InTimecode.Frames) / Divider);
		Timecode.bIsDropFrame = InTimecode.bDropFrameFormat;
		return Timecode;
	}
}

///* UBlackmagicMediaCapture implementation
//*****************************************************************************/
UBlackmagicMediaCapture::UBlackmagicMediaCapture()
	: bWaitForSyncEvent(false)
	, bLogDropFrame(false)
	, BlackmagicMediaOutputPixelFormat(EBlackmagicMediaOutputPixelFormat::PF_8BIT_YUV)
	, bSavedIgnoreTextureAlpha(false)
	, bIgnoreTextureAlphaChanged(false)
	, FrameRate(30, 1)
	, WakeUpEvent(nullptr)
	, LastFrameDropCount_BlackmagicThread(0)
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		if (FGPUTextureTransferModule::Get().IsInitialized())
		{
			TextureTransfer = FGPUTextureTransferModule::Get().GetTextureTransfer();
		}
	}
}

bool UBlackmagicMediaCapture::ValidateMediaOutput() const
{
	UBlackmagicMediaOutput* BlackmagicMediaOutput = Cast<UBlackmagicMediaOutput>(MediaOutput);
	if (!BlackmagicMediaOutput)
	{
		UE_LOGF(LogBlackmagicMediaOutput, Error, "Can not start the capture. MediaOutput's class is not supported.");
		return false;
	}

	return true;
}

bool UBlackmagicMediaCapture::InitializeCapture()
{
	UBlackmagicMediaOutput* BlackmagicMediaOutput = CastChecked<UBlackmagicMediaOutput>(MediaOutput);
	BlackmagicMediaOutputPixelFormat = BlackmagicMediaOutput->PixelFormat;

	if (FGPUTextureTransferModule::Get().IsEnabled() && !FGPUTextureTransferModule::Get().IsInitialized())
	{
		// This will trigger an initialization of GPUDirect.
		TextureTransfer = FGPUTextureTransferModule::Get().GetTextureTransfer();
	}

	bool bInitialized = InitBlackmagic(BlackmagicMediaOutput);
	return bInitialized;
}

bool UBlackmagicMediaCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	ApplyViewportTextureAlpha(InSceneViewport);
	return true;
}

bool UBlackmagicMediaCapture::UpdateSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	RestoreViewportTextureAlpha(GetCapturingSceneViewport());
	ApplyViewportTextureAlpha(InSceneViewport);
	return true;
}

bool UBlackmagicMediaCapture::UpdateRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget)
{
	RestoreViewportTextureAlpha(GetCapturingSceneViewport());
	return true;
}

bool UBlackmagicMediaCapture::UpdateAudioDeviceImpl(const FAudioDeviceHandle& InAudioDeviceHandle)
{
	return CreateAudioOutput(InAudioDeviceHandle, Cast<UBlackmagicMediaOutput>(MediaOutput));
}

void UBlackmagicMediaCapture::StopCaptureImpl(bool bAllowPendingFrameToBeProcess)
{
	if (!bAllowPendingFrameToBeProcess)
	{
		{
			// Prevent the rendering thread from copying while we are stopping the capture.
			FScopeLock ScopeLock(&CopyingCriticalSection);
			ENQUEUE_RENDER_COMMAND(BlackmagicMediaCaptureInitialize)(
				[this](FRHICommandListImmediate& RHICmdList) mutable
				{
					for (FTextureRHIRef& Texture : TexturesToRelease)
					{
						TextureTransfer->UnregisterTexture(Texture->GetTexture2D());
					}

					TexturesToRelease.Reset();

					if (EventCallback)
					{
						EventCallback->Uninitialize();
						EventCallback = nullptr;
					}
				});

			if (WakeUpEvent)
			{
				FPlatformProcess::ReturnSynchEventToPool(WakeUpEvent);
				WakeUpEvent = nullptr;
			}
		}

		RestoreViewportTextureAlpha(GetCapturingSceneViewport());

		AudioOutput.Reset();
	}
}

bool UBlackmagicMediaCapture::ShouldCaptureRHIResource() const
{
	return TextureTransfer && FGPUTextureTransferModule::Get().IsEnabled();
}

void UBlackmagicMediaCapture::ApplyViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport)
{
	if (InSceneViewport.IsValid())
	{
		TSharedPtr<SViewport> Widget(InSceneViewport->GetViewportWidget().Pin());
		if (Widget.IsValid())
		{
			bSavedIgnoreTextureAlpha = Widget->GetIgnoreTextureAlpha();

			UBlackmagicMediaOutput* BlackmagicMediaOutput = CastChecked<UBlackmagicMediaOutput>(MediaOutput);
			if (BlackmagicMediaOutput->OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey)
			{
				if (bSavedIgnoreTextureAlpha)
				{
					bIgnoreTextureAlphaChanged = true;
					Widget->SetIgnoreTextureAlpha(false);
				}
			}
		}
	}
}

void UBlackmagicMediaCapture::RestoreViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport)
{
	// restore the ignore texture alpha state
	if (bIgnoreTextureAlphaChanged)
	{
		if (InSceneViewport.IsValid())
		{
			TSharedPtr<SViewport> Widget(InSceneViewport->GetViewportWidget().Pin());
			if (Widget.IsValid())
			{
				Widget->SetIgnoreTextureAlpha(bSavedIgnoreTextureAlpha);
			}
		}
		bIgnoreTextureAlphaChanged = false;
	}
}

void UBlackmagicMediaCapture::OnFrameCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData)
{
	// Prevent this thread from copying while we are stopping the capture.
	FScopeLock ScopeLock(&CopyingCriticalSection);
	
	if (EventCallback)
	{
		BlackmagicDesign::FTimecode Timecode = BlackmagicMediaCaptureDevice::ConvertToBlackmagicTimecode(InBaseData.SourceFrameTimecode, InBaseData.SourceFrameTimecodeFramerate.AsDecimal(), FrameRate.AsDecimal());

		BlackmagicDesign::FFrameDescriptor Frame;
		Frame.VideoBuffer = reinterpret_cast<uint8_t*>(InResourceData.Buffer);
		Frame.VideoWidth = InResourceData.Width;
		Frame.VideoHeight = InResourceData.Height;
		Frame.Timecode = Timecode;
		Frame.FrameIdentifier = InBaseData.SourceFrameNumber;
		Frame.bEvenFrame = GFrameCounterRenderThread % 2 == 0;

		bool bSent = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::SendVideoFrameData);
			bSent = EventCallback->SendVideoFrameData(Frame);
		}
		
		
		if (bLogDropFrame && !bSent)
		{
			UE_LOGF(LogBlackmagicMediaOutput, Warning, "Frame couldn't be sent to Blackmagic device. Engine might be running faster than output.");
		}

		OutputAudio_AnyThread(InBaseData, Timecode);

		if (bBlackmagicWritInputRawDataCmdEnable)
		{
			FString OutputFilename;
			uint32 Stride = 0;

			switch (BlackmagicMediaOutputPixelFormat)
			{
			case EBlackmagicMediaOutputPixelFormat::PF_8BIT_YUV:
				if (GetConversionOperation() == EMediaCaptureConversionOperation::RGBA8_TO_YUV_8BIT)
				{
					OutputFilename = TEXT("Blackmagic_Input_8_YUV");
					Stride = InResourceData.Width * 4;
					break;
				}
				else
				{
					OutputFilename = TEXT("Blackmagic_Input_8_RGBA");
					Stride = InResourceData.Width * 4;
					break;
				}
			case EBlackmagicMediaOutputPixelFormat::PF_10BIT_YUV:
				OutputFilename = TEXT("Blackmagic_Input_10_YUV");
				Stride = InResourceData.Width * 16;
				break;
			}

			MediaIOCoreFileWriter::WriteRawFile(OutputFilename, reinterpret_cast<uint8*>(InResourceData.Buffer), Stride * InResourceData.Height);
			bBlackmagicWritInputRawDataCmdEnable = false;
		}


		WaitForSync_AnyThread();
	}
	else if (GetState() != EMediaCaptureState::Stopped)
	{
		SetState(EMediaCaptureState::Error);
	}
}

bool UBlackmagicMediaCapture::HasFinishedProcessing() const
{
	return Super::HasFinishedProcessing() || EventCallback == nullptr;
}

const FMatrix& UBlackmagicMediaCapture::GetRGBToYUVConversionMatrix() const
{
	if (const UBlackmagicMediaOutput* BMOutput = Cast<UBlackmagicMediaOutput>(MediaOutput))
	{
		switch(BMOutput->HDROptions.Gamut)
		{
		case EBlackmagicHDRMetadataGamut::Rec709:
			return MediaShaders::RgbToYuvRec709Scaled;
		case EBlackmagicHDRMetadataGamut::Rec2020:
			return MediaShaders::RgbToYuvRec2020Scaled;
		default:
			checkNoEntry();
			return MediaShaders::RgbToYuvRec709Scaled;
		}
	}
		
	return Super::GetRGBToYUVConversionMatrix();
}

void UBlackmagicMediaCapture::WaitForGPU(FRHITexture* InRHITexture)
{
	if (TextureTransfer)
	{
		TextureTransfer->WaitForGPU(InRHITexture);
	}
}

bool UBlackmagicMediaCapture::ShouldCaptureThisFrame(const FCaptureBaseData& InBaseData) const
{
	if (EventCallback)
	{
		return EventCallback->ShouldCaptureThisFrame(InBaseData.SourceFrameTimecode, InBaseData.SourceFrameNumber);
	}

	return false;
}

#if WITH_EDITOR
FString UBlackmagicMediaCapture::GetAnalyticsEventPrefix() const
{
	return TEXT("MediaFramework.Blackmagic");
}

void UBlackmagicMediaCapture::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	Super::GetAnalyticsEventAttributes(EventName, InOutAttributes);

	UBlackmagicMediaOutput* BlackmagicMediaOutput = CastChecked<UBlackmagicMediaOutput>(MediaOutput);

	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Framerate"), BlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.FrameRate.ToPrettyText().ToString()));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Device"), BlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName));

	if (IMediaIOCoreDeviceProvider* DeviceProvider = IMediaIOCoreModule::Get().GetDeviceProvider(FBlackmagicDeviceProvider::GetProviderName()))
	{
		const FString DestStr = FString::Format(TEXT("{0} {1}"), {
			DeviceProvider->GetTransportName(
				BlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.TransportType,
				BlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.QuadTransportType).ToString(),
			BlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.PortIdentifier });
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Destination"), DestStr));
	}

	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Standard"), UEnum::GetValueAsString(BlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Standard)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Reference"), UEnum::GetValueAsString(BlackmagicMediaOutput->OutputConfiguration.OutputReference)));
}
#endif

bool UBlackmagicMediaCapture::InitBlackmagic(UBlackmagicMediaOutput* InBlackmagicMediaOutput)
{
	check(InBlackmagicMediaOutput);

	IBlackmagicMediaModule& MediaModule = FModuleManager::LoadModuleChecked<IBlackmagicMediaModule>(TEXT("BlackmagicMedia"));
	if (!MediaModule.CanBeUsed())
	{
		UE_LOGF(LogBlackmagicMediaOutput, Warning, "The BlackmagicMediaCapture can't open MediaOutput '%ls' because Blackmagic card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceBlackmagicUsage", *InBlackmagicMediaOutput->GetName());
		return false;
	}

	// Init general settings
	bWaitForSyncEvent = InBlackmagicMediaOutput->bWaitForSyncEvent;
	bLogDropFrame = InBlackmagicMediaOutput->bLogDropFrame;
	FrameRate = InBlackmagicMediaOutput->GetRequestedFrameRate();

	if (ShouldCaptureRHIResource())
	{
		if (InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Standard != EMediaIOStandardType::Progressive)
		{
			UE_LOGF(LogBlackmagicMediaOutput, Warning, "GPU DMA is not supported with interlaced, defaulting to regular path.");
			TextureTransfer.Reset();
		}
	}


	// Init Device options
	BlackmagicDesign::FOutputChannelOptions ChannelOptions;
	ChannelOptions.FormatInfo.DisplayMode = InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier;
	
	ChannelOptions.FormatInfo.Width = InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Resolution.X;
	ChannelOptions.FormatInfo.Height = InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Resolution.Y;
	ChannelOptions.FormatInfo.FrameRateNumerator = InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.FrameRate.Numerator;
	ChannelOptions.FormatInfo.FrameRateDenominator = InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.FrameRate.Denominator;
	ChannelOptions.bOutputAudio = InBlackmagicMediaOutput->bOutputAudio;
	ChannelOptions.AudioBitDepth = BlackmagicMediaCaptureHelpers::ConvertAudioBitDepth(InBlackmagicMediaOutput->AudioBitDepth);
	ChannelOptions.NumAudioChannels = static_cast<BlackmagicDesign::EAudioChannelConfiguration>(InBlackmagicMediaOutput->OutputChannelCount);
	ChannelOptions.AudioSampleRate = static_cast<BlackmagicDesign::EAudioSampleRate>(InBlackmagicMediaOutput->AudioSampleRate);

	ChannelOptions.FormatInfo.FieldDominance = BlackmagicMediaCaptureHelpers::GetFieldDominanceFromMediaStandard(InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Standard);
	ChannelOptions.PixelFormat = BlackmagicMediaCaptureHelpers::ConvertPixelFormat(InBlackmagicMediaOutput->PixelFormat);
	ChannelOptions.TimecodeFormat = BlackmagicMediaCaptureHelpers::ConvertTimecodeFormat(InBlackmagicMediaOutput->TimecodeFormat);
	ChannelOptions.LinkConfiguration = BlackmagicMediaCaptureHelpers::ConvertTransportType(InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.TransportType, InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.QuadTransportType);

	ChannelOptions.bOutputKey = InBlackmagicMediaOutput->OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey;
	ChannelOptions.NumberOfBuffers = FMath::Clamp(InBlackmagicMediaOutput->NumberOfBlackmagicBuffers, 3, 4);
	ChannelOptions.bOutputVideo = true;
	ChannelOptions.bOutputInterlacedFieldsTimecodeNeedToMatch = InBlackmagicMediaOutput->bInterlacedFieldsTimecodeNeedToMatch && InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Standard == EMediaIOStandardType::Interlaced && InBlackmagicMediaOutput->TimecodeFormat != EMediaIOTimecodeFormat::None;
	ChannelOptions.bOutputInterlaceAsProgressive = InBlackmagicMediaOutput->bOutputInterlaceAsProgressive;
	ChannelOptions.bOutputInterlaceOnEvenFrames = InBlackmagicMediaOutput->bOutputInterlaceOnEvenFrames;
	ChannelOptions.bLogDropFrames = bLogDropFrame;
	ChannelOptions.bUseGPUDMA = ShouldCaptureRHIResource();
	ChannelOptions.bScheduleInDifferentThread = InBlackmagicMediaOutput->bUseMultithreadedScheduling;
	ChannelOptions.HDRMetadata = BlackmagicMediaCaptureHelpers::MakeBlackmagicHDRMetadata(InBlackmagicMediaOutput->HDROptions);
	ChannelOptions.bEncodeTimecodeInTexel = InBlackmagicMediaOutput->bEncodeTimecodeInTexel;

	AudioBitDepth = InBlackmagicMediaOutput->AudioBitDepth;
	bOutputAudio = InBlackmagicMediaOutput->bOutputAudio;
	NumOutputChannels = static_cast<uint8>(InBlackmagicMediaOutput->OutputChannelCount);

	if (bOutputAudio)
	{
		if (!CreateAudioOutput(AudioDeviceHandle, InBlackmagicMediaOutput))
		{
			UE_LOGF(LogBlackmagicMediaOutput, Error, "Failed to initialize audio output.");
		}
	}
	
	check(EventCallback == nullptr);
	BlackmagicDesign::FChannelInfo ChannelInfo;
	ChannelInfo.DeviceIndex = InBlackmagicMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier;
	EventCallback = new BlackmagicMediaCaptureHelpers::FBlackmagicMediaCaptureEventCallback(this, ChannelInfo);

	const bool bSuccess = EventCallback->Initialize(ChannelOptions);
	if (!bSuccess)
	{
		UE_LOGF(LogBlackmagicMediaOutput, Warning, "The Blackmagic output port for '%ls' could not be opened.", *InBlackmagicMediaOutput->GetName());
		EventCallback->Uninitialize();
		EventCallback = nullptr;
		return false;
	}

	if (bSuccess && bWaitForSyncEvent)
	{
		const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
		bool bLockToVsync = CVar->GetValueOnGameThread() != 0;
		if (bLockToVsync)
		{
			UE_LOGF(LogBlackmagicMediaOutput, Warning, "The Engine use VSync and something to wait for the sync event. This may break the \"gen-lock\".");
		}

		const bool bIsManualReset = false;
		WakeUpEvent = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
	}

	if (ShouldCaptureRHIResource())
	{
		UE_LOGF(LogBlackmagicMediaOutput, Display, "BlackmagicMedia capture started using GPU Direct");
	}

	return true;
}

bool UBlackmagicMediaCapture::CreateAudioOutput(const FAudioDeviceHandle& InAudioDeviceHandle, const UBlackmagicMediaOutput* InBlackmagicMediaOutput)
{
	if (GEngine && bOutputAudio && InBlackmagicMediaOutput)
	{
		UMediaIOCoreSubsystem::FCreateAudioOutputArgs Args;
		Args.NumOutputChannels = static_cast<int32>(InBlackmagicMediaOutput->OutputChannelCount);
		Args.TargetFrameRate = FrameRate;
		Args.MaxSampleLatency = Align(InBlackmagicMediaOutput->AudioBufferSize, 4);
		Args.OutputSampleRate = static_cast<uint32>(InBlackmagicMediaOutput->AudioSampleRate);
		Args.AudioDeviceHandle = InAudioDeviceHandle;
		AudioOutput = GEngine->GetEngineSubsystem<UMediaIOCoreSubsystem>()->CreateAudioOutput(Args);
		return AudioOutput.IsValid();
	}
	return false;
}

void UBlackmagicMediaCapture::LockDMATexture_RenderThread(FTextureRHIRef InTexture)
{
	if (ShouldCaptureRHIResource() && TextureTransfer)
	{
		if (!TexturesToRelease.Contains(InTexture))
		{
			TexturesToRelease.Add(InTexture);

			FRHITexture* Texture = InTexture->GetTexture2D();
			UE::GPUTextureTransfer::FRegisterDMATextureArgs Args;
			Args.RHITexture = Texture;

			Args.RHIResourceMemory = nullptr; // = Texture->GetNativeResource(); todo: VulkanTexture->Surface->GetAllocationHandle for Vulkan
			Args.Width = Texture->GetDesc().GetSize().X;
			Args.Height = Texture->GetDesc().GetSize().Y;

			if (Args.RHITexture->GetFormat() == EPixelFormat::PF_B8G8R8A8)
			{
				Args.PixelFormat = UE::GPUTextureTransfer::EPixelFormat::PF_8Bit;
				Args.Stride = Args.Width * 4;
			}
			else if (Args.RHITexture->GetFormat() == EPixelFormat::PF_R32G32B32A32_UINT)
			{
				Args.PixelFormat = UE::GPUTextureTransfer::EPixelFormat::PF_10Bit;
				Args.Stride = Args.Width * 16;
			}
			else
			{
				checkf(false, TEXT("Format not supported"));
			}
			//Args.RHIResourceMemory = Texture->GetNativeResource(); todo: VulkanTexture->Surface->GetAllocationHandle for Vulkan

			TextureTransfer->RegisterTexture(Args);

		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::LockDMATexture);
			TextureTransfer->LockTexture(InTexture->GetTexture2D());
		}
	}
}

void UBlackmagicMediaCapture::UnlockDMATexture_RenderThread(FTextureRHIRef InTexture)
{
	if (TextureTransfer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::UnlockDMATexture);
		TextureTransfer->UnlockTexture(InTexture->GetTexture2D());
	}
}

bool UBlackmagicMediaCapture::SupportsAnyThreadCapture() const
{
	return true;
}

void UBlackmagicMediaCapture::OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::OnFrameCaptured_RenderingThread);
	FMediaCaptureResourceData ResourceData;
	ResourceData.Buffer = InBuffer;
	ResourceData.Width = Width;
	ResourceData.Height = Height;
	ResourceData.BytesPerRow = BytesPerRow;
	
	OnFrameCaptured_AnyThread(InBaseData, InUserData, ResourceData);
}

void UBlackmagicMediaCapture::OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::OnFrameCaptured_AnyThread);

	OnFrameCapturedInternal_AnyThread(InBaseData, InUserData, InResourceData);
}

void UBlackmagicMediaCapture::OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& /*RHICmdList*/, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::OnRHIResourceCaptured_RenderingThread);
	OnRHIResourceCaptured_AnyThread(InBaseData, InUserData, InTexture);
}

void UBlackmagicMediaCapture::OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture)
{
	if (!InTexture)
	{
		return;
	}
	
	// Prevent the rendering thread from copying while we are stopping the capture.
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::OnRHIResourceCaptured_AnyThread);
	FScopeLock ScopeLock(&CopyingCriticalSection);

	if (EventCallback)
	{
		BlackmagicDesign::FTimecode Timecode = BlackmagicMediaCaptureDevice::ConvertToBlackmagicTimecode(InBaseData.SourceFrameTimecode, InBaseData.SourceFrameTimecodeFramerate.AsDecimal(), FrameRate.AsDecimal());

		BlackmagicDesign::FFrameDescriptor_GPUDMA Frame;
		Frame.RHITexture = InTexture->GetTexture2D();
		Frame.Timecode = Timecode;
		Frame.FrameIdentifier = InBaseData.SourceFrameNumber;
		Frame.bEvenFrame = GFrameCounterRenderThread % 2 == 0;

		bool bSent = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::SendVideoFrameData_GPUDirect);
			bSent = EventCallback->SendVideoFrameData(Frame);
		}

		if (bLogDropFrame && !bSent)
		{
			UE_LOGF(LogBlackmagicMediaOutput, Warning, "Frame couldn't be sent to Blackmagic device. Engine might be running faster than output.");
		}

		OutputAudio_AnyThread(InBaseData, Timecode);

		WaitForSync_AnyThread();
	}
	else if (GetState() != EMediaCaptureState::Stopped)
	{
		SetState(EMediaCaptureState::Error);
	}
}

void UBlackmagicMediaCapture::WaitForSync_AnyThread()
{
	if (bWaitForSyncEvent)
	{
		if (WakeUpEvent && GetState() == EMediaCaptureState::Capturing) // Could be shutdown in a middle of a frame 
		{
			const uint32 NumberOfMilliseconds = 1000;
			if (!WakeUpEvent->Wait(NumberOfMilliseconds))
			{
				SetState(EMediaCaptureState::Error);
				UE_LOGF(LogBlackmagicMediaOutput, Error, "Could not synchronize with the device.");
			}
		}
	}
}

void UBlackmagicMediaCapture::OutputAudio_AnyThread(const FCaptureBaseData& InBaseData, const BlackmagicDesign::FTimecode& InTimecode)
{
	if (bOutputAudio)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UBlackmagicMediaCapture::OutputAudio);

		// Take a local copy of the audio output in case it is switched from the main thread.
		const TSharedPtr<FMediaIOAudioOutput> LocalAudioOutput = AudioOutput;
		if(!LocalAudioOutput)
		{
			return;
		}

		const double NewTimestamp = FPlatformTime::Seconds();
		double CurrentFrameTime = NewTimestamp - OutputAudioTimestamp;
		
		const float TargetFrametime = 1 / FrameRate.AsDecimal();
		if (UNLIKELY(OutputAudioTimestamp == 0))
		{
			CurrentFrameTime = TargetFrametime;
		}

		float FrameTimeRatio = CurrentFrameTime / TargetFrametime;

		uint32_t NumSamplesToPull = FrameTimeRatio * LocalAudioOutput->NumSamplesPerFrame;
		NumSamplesToPull = FMath::Clamp<uint32_t>(NumSamplesToPull, 0, LocalAudioOutput->NumSamplesPerFrame);

		BlackmagicDesign::FAudioSamplesDescriptor AudioSamples;
		AudioSamples.Timecode = InTimecode;
		AudioSamples.FrameIdentifier = InBaseData.SourceFrameNumber;
		check(NumOutputChannels != 0);
		
		if (AudioBitDepth == EBlackmagicMediaOutputAudioBitDepth::Signed_32Bits)
		{
			TArray<int32> AudioBuffer = LocalAudioOutput->GetAllAudioSamples<int32>();
			AudioSamples.AudioBuffer = reinterpret_cast<uint8_t*>(AudioBuffer.GetData());
			AudioSamples.NumAudioSamples = AudioBuffer.Num() / NumOutputChannels;
			AudioSamples.AudioBufferLength = AudioBuffer.Num() * sizeof(int32);
			EventCallback->SendAudioSamples(AudioSamples);
		}
		else if (AudioBitDepth == EBlackmagicMediaOutputAudioBitDepth::Signed_16Bits)
		{
			TArray<int16> AudioBuffer = LocalAudioOutput->GetAllAudioSamples<int16>();
			AudioSamples.AudioBuffer = reinterpret_cast<uint8_t*>(AudioBuffer.GetData());
			AudioSamples.NumAudioSamples = AudioBuffer.Num() / NumOutputChannels;
			AudioSamples.AudioBufferLength = AudioBuffer.Num() * sizeof(int16);

			EventCallback->SendAudioSamples(AudioSamples);
		}
		else
		{
			checkNoEntry();
		}
		
		OutputAudioTimestamp = NewTimestamp;
	}
}
