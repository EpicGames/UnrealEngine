// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicMediaOutput.h"

#include "BlackmagicDeviceProvider.h"
#include "BlackmagicLib.h"
#include "BlackmagicMediaCapture.h"
#include "IBlackmagicMediaModule.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "Modules/ModuleManager.h"


#define LOCTEXT_NAMESPACE "BlackmagicMediaOutput"


/* UBlackmagicMediaOutput
*****************************************************************************/

UBlackmagicMediaOutput::UBlackmagicMediaOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TimecodeFormat(EMediaIOTimecodeFormat::LTC)
	, PixelFormat(EBlackmagicMediaOutputPixelFormat::PF_8BIT_YUV)
	, bInvertKeyOutput(false)
	, bOutputAudio(false)
	, NumberOfBlackmagicBuffers(3)
	, bInterlacedFieldsTimecodeNeedToMatch(false)
	, bWaitForSyncEvent(false)
	, bLogDropFrame(false)
	, bEncodeTimecodeInTexel(false)
{
}

bool UBlackmagicMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}

	if (!OutputConfiguration.IsValid())
	{
		OutFailureReason = FString::Printf(TEXT("The Configuration of '%s' is invalid."), *GetName());
		return false;
	}

	IBlackmagicMediaModule& MediaModule = FModuleManager::LoadModuleChecked<IBlackmagicMediaModule>(TEXT("BlackmagicMedia"));
	if (!MediaModule.IsInitialized())
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s'. The Blackmagic library was not initialized."), *GetName());
		return false;
	}

	if (!MediaModule.CanBeUsed())
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s' because Blackmagic card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceBlackmagicUsage."), *GetName());
		return false;
	}

	BlackmagicDesign::BlackmagicDeviceScanner Scanner;
	BlackmagicDesign::BlackmagicDeviceScanner::DeviceInfo DeviceInfo;
	if (!Scanner.GetDeviceInfo(OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier, DeviceInfo))
	{
		OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that doesn't exist on this machine."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (!DeviceInfo.bIsSupported)
	{
		OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that is not supported by the Blackmagic SDK."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (!DeviceInfo.bCanDoPlayback)
	{
		OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that can't do playback."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey && PixelFormat == EBlackmagicMediaOutputPixelFormat::PF_10BIT_YUV)
	{
		OutFailureReason = FString::Printf(TEXT("'%s', Blackmagic devices do not support 10bit key."), *GetName());
		return false;
	}

	return true;
}

FFrameRate UBlackmagicMediaOutput::GetRequestedFrameRate() const
{
	return OutputConfiguration.MediaConfiguration.MediaMode.FrameRate;
}

FIntPoint UBlackmagicMediaOutput::GetRequestedSize() const
{
	return OutputConfiguration.MediaConfiguration.MediaMode.Resolution;
}

EPixelFormat UBlackmagicMediaOutput::GetRequestedPixelFormat() const
{
	EPixelFormat Result = EPixelFormat::PF_A2B10G10R10;
	switch (PixelFormat)
	{
	case EBlackmagicMediaOutputPixelFormat::PF_8BIT_YUV:
		Result = EPixelFormat::PF_B8G8R8A8;
		break;
	case EBlackmagicMediaOutputPixelFormat::PF_10BIT_YUV:
		Result = EPixelFormat::PF_A2B10G10R10;
		break;
	}
	return Result;
}

EMediaCaptureConversionOperation UBlackmagicMediaOutput::GetConversionOperation(EMediaCaptureSourceType InSourceType) const
{
	EMediaCaptureConversionOperation Result = EMediaCaptureConversionOperation::NONE;

	switch (PixelFormat)
	{
	case EBlackmagicMediaOutputPixelFormat::PF_8BIT_YUV:
		if (OutputConfiguration.OutputType == EMediaIOOutputType::Fill)
		{
			Result = EMediaCaptureConversionOperation::RGBA8_TO_YUV_8BIT;
		}
		else if (OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey && bInvertKeyOutput)
		{
			Result = EMediaCaptureConversionOperation::INVERT_ALPHA;
		}
		else
		{
			Result = EMediaCaptureConversionOperation::NONE;
		}
		break;
	case EBlackmagicMediaOutputPixelFormat::PF_10BIT_YUV:
		Result = EMediaCaptureConversionOperation::RGB10_TO_YUVv210_10BIT;
		break;
	}
	return Result;
}

#if WITH_EDITOR
FString UBlackmagicMediaOutput::GetDescriptionString() const
{
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FBlackmagicDeviceProvider::GetProviderName());
	if (DeviceProviderPtr)
	{
		return DeviceProviderPtr->ToText(OutputConfiguration).ToString(); 
	}

	return Super::GetDescriptionString();
}

void UBlackmagicMediaOutput::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FBlackmagicDeviceProvider::GetProviderName());
	if (!DeviceProviderPtr)
	{
		return;
	}
	
	const FText ConfigHeader = LOCTEXT("MediaConfigurationHeader", "Media Configuration");

	const FText SourceText = FText::Format(LOCTEXT("TransportFormat", "{0} {1}"),
		DeviceProviderPtr->GetTransportName(OutputConfiguration.MediaConfiguration.MediaConnection.TransportType, OutputConfiguration.MediaConfiguration.MediaConnection.QuadTransportType),
		FText::AsNumber(OutputConfiguration.MediaConfiguration.MediaConnection.PortIdentifier));
	
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("DeviceLabel", "Device"), FText::FromName(OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("SourceLabel", "Source"), SourceText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("ResolutionLabel", "Resolution"), OutputConfiguration.MediaConfiguration.MediaMode.GetModeName()));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("StandardLabel", "Standard"), UEnum::GetDisplayValueAsText(OutputConfiguration.MediaConfiguration.MediaMode.Standard)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("FramerateLabel", "Framerate"), OutputConfiguration.MediaConfiguration.MediaMode.FrameRate.ToPrettyText()));

	const FText AdditionalHeader = LOCTEXT("AdditionalSettingsHeader", "Additional Settings");
	const FText DisabledText = LOCTEXT("Disabled", "Disabled");
	const FText AudioText = FText::Format(LOCTEXT("AudioFormat", "{0} channels, {1} sample rate"),
		UEnum::GetDisplayValueAsText(OutputChannelCount), UEnum::GetDisplayValueAsText(AudioSampleRate));
	
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("TimecodeFormatLabel", "Timecode Format"), UEnum::GetDisplayValueAsText(TimecodeFormat)));
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("AudioLabel", "Audio"), bOutputAudio ? AudioText : DisabledText));
}
#endif //WITH_EDITOR

UMediaCapture* UBlackmagicMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<UBlackmagicMediaCapture>();
	if (Result)
	{
		Result->SetMediaOutput(this);
	}
	return Result;
}

#if WITH_EDITOR
bool UBlackmagicMediaOutput::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackmagicMediaOutput, bEncodeTimecodeInTexel))
	{
		return TimecodeFormat != EMediaIOTimecodeFormat::None;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackmagicMediaOutput, bInvertKeyOutput))
	{
		return (PixelFormat == EBlackmagicMediaOutputPixelFormat::PF_8BIT_YUV && OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey);
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackmagicMediaOutput, bInterlacedFieldsTimecodeNeedToMatch))
	{
		bool bValid = false;
		if (OutputConfiguration.IsValid() && TimecodeFormat != EMediaIOTimecodeFormat::None)
		{
			bValid = OutputConfiguration.MediaConfiguration.MediaMode.Standard == EMediaIOStandardType::Interlaced;
		}
		return bValid;
	}

	return true;
}

void UBlackmagicMediaOutput::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackmagicMediaOutput, TimecodeFormat))
	{
		if (TimecodeFormat == EMediaIOTimecodeFormat::None)
		{
			bEncodeTimecodeInTexel = false;
			bInterlacedFieldsTimecodeNeedToMatch = false;
		}
	}

	if (InPropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackmagicMediaOutput, OutputConfiguration))
	{
		if (OutputConfiguration.OutputType == EMediaIOOutputType::Fill)
		{
			bInvertKeyOutput = false;
		}


		if (bInterlacedFieldsTimecodeNeedToMatch)
		{
			bInterlacedFieldsTimecodeNeedToMatch = false;
			if (OutputConfiguration.IsValid() && TimecodeFormat != EMediaIOTimecodeFormat::None)
			{
				bInterlacedFieldsTimecodeNeedToMatch = OutputConfiguration.MediaConfiguration.MediaMode.Standard == EMediaIOStandardType::Interlaced;;
			}
		}
	}

	Super::PostEditChangeChainProperty(InPropertyChangedEvent);
}

#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE
