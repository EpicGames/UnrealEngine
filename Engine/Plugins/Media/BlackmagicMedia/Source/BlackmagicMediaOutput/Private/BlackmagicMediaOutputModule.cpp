// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackmagicMediaOutputModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogBlackmagicMediaOutput);

IMPLEMENT_MODULE(FDefaultModuleImpl, BlackmagicMediaOutput)
