// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"
#include "HAL/CriticalSection.h"
#include "MediaIOCoreEncodeTime.h"
#include "Misc/FrameRate.h"
#include "BlackmagicMediaOutput.h"
#include "BlackmagicMediaCapture.generated.h"

class FEvent;


namespace BlackmagicMediaCaptureHelpers
{
	class FBlackmagicMediaCaptureEventCallback;
}

namespace BlackmagicDesign
{
	struct FTimecode;
}

namespace UE::GPUTextureTransfer
{
	class ITextureTransfer;
}

/**
 * Output Media for Blackmagic streams.
 * The output format could be any of EBlackmagicMediaOutputPixelFormat.
 */
UCLASS(BlueprintType)
class BLACKMAGICMEDIAOUTPUT_API UBlackmagicMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

public:
	UBlackmagicMediaCapture();
	//~ UMediaCapture interface
public:
	virtual bool HasFinishedProcessing() const override;
protected:
	virtual bool ValidateMediaOutput() const override;
	virtual bool InitializeCapture() override;
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool UpdateSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool UpdateRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget) override;
	virtual bool UpdateAudioDeviceImpl(const FAudioDeviceHandle& InAudioDeviceHandle) override;
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;
	virtual bool ShouldCaptureRHIResource() const override;

	virtual void OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow) override;
	virtual void OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData) override;
	virtual void OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) override;
	virtual void OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) override;
	virtual void LockDMATexture_RenderThread(FTextureRHIRef InTexture) override;
	virtual void UnlockDMATexture_RenderThread(FTextureRHIRef InTexture) override;
	virtual bool SupportsAnyThreadCapture() const override;
	virtual const FMatrix& GetRGBToYUVConversionMatrix() const override;
	virtual void WaitForGPU(FRHITexture* InRHITexture) override;
	virtual bool ShouldCaptureThisFrame(const FCaptureBaseData& InBaseData) const override;
	
#if WITH_EDITOR
	virtual FString GetAnalyticsEventPrefix() const override;
	virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const override;
#endif

private:
	struct FBlackmagicOutputCallback;
	friend FBlackmagicOutputCallback;

private:
	bool InitBlackmagic(UBlackmagicMediaOutput* InMediaOutput);
	bool CreateAudioOutput(const FAudioDeviceHandle& InAudioDeviceHandle, const UBlackmagicMediaOutput* InBlackmagicMediaOutput);
	void WaitForSync_AnyThread();
	void OutputAudio_AnyThread(const FCaptureBaseData& InBaseData, const BlackmagicDesign::FTimecode& Timecode);
	void ApplyViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport);
	void RestoreViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport);
	void OnFrameCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData);

private:
	friend BlackmagicMediaCaptureHelpers::FBlackmagicMediaCaptureEventCallback;
	BlackmagicMediaCaptureHelpers::FBlackmagicMediaCaptureEventCallback* EventCallback = nullptr;

	/** Option from MediaOutput */
	bool bWaitForSyncEvent;
	bool bLogDropFrame;
	
	/** MediaOutput cached value */
	EBlackmagicMediaOutputPixelFormat BlackmagicMediaOutputPixelFormat;

	/** Saved IgnoreTextureAlpha flag from viewport */
	bool bSavedIgnoreTextureAlpha;
	bool bIgnoreTextureAlphaChanged;

	/** Selected FrameRate of this output */
	FFrameRate FrameRate;

	/** Critical section for synchronizing access to the OutputChannel */
	FCriticalSection CopyingCriticalSection;

	/** Event to wakeup When waiting for sync */
	FEvent* WakeUpEvent;

	/** Last frame drop count to detect count */
	uint64 LastFrameDropCount_BlackmagicThread;

	/** Holds an audio output that will receive samples from the media io core subsystem. */
	TSharedPtr<class FMediaIOAudioOutput> AudioOutput;

	EBlackmagicMediaOutputAudioBitDepth AudioBitDepth = EBlackmagicMediaOutputAudioBitDepth::Signed_16Bits;

	uint32 NumAudioSamplesPerFrame = 0;

	bool bOutputAudio = false;

	uint8 NumOutputChannels = 0;

	/** Textures to release when the capture has stopped. Must be released after GPUTextureTransfer textures have been unregistered. */
	TArray<FTextureRHIRef> TexturesToRelease;

	/** The last time OutputAudio was called. Used to adjust the number of audio samples to grab on each frame depending on the frame time. */
	double OutputAudioTimestamp;

	TSharedPtr<UE::GPUTextureTransfer::ITextureTransfer> TextureTransfer;
};
