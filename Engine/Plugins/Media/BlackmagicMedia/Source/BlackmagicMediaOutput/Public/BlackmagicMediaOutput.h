// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"

#include "BlackmagicMediaDefinitions.h"
#include "MediaIOCoreDefinitions.h"

#include "BlackmagicMediaOutput.generated.h"

/**
 * Native data format.
 */
UENUM()
enum class EBlackmagicMediaOutputPixelFormat : uint8
{
	PF_8BIT_YUV UMETA(DisplayName = "8bit YUV"),
	PF_10BIT_YUV UMETA(DisplayName = "10bit YUV"),
};

UENUM()
enum class EBlackmagicMediaOutputAudioSampleRate : uint32
{
	SR_48k = 48000 UMETA(DisplayName = "48 kHz")
};

UENUM()
enum class EBlackmagicMediaAudioOutputChannelCount : uint8
{
	CH_2  =  2 UMETA(DisplayName = "2"),
	CH_8  =  8 UMETA(DisplayName = "8"),
	CH_16 = 16 UMETA(DisplayName = "16")
};

UENUM()
enum class EBlackmagicMediaOutputAudioBitDepth : uint8
{
	Signed_16Bits = 16 UMETA(DisplayName = "16 bits signed"),
	Signed_32Bits = 32 UMETA(DisplayName = "32 bits signed")
};

/**
 * Output information for a MediaCapture.
 * @note	'Frame Buffer Pixel Format' must be set to at least 8 bits of alpha to enabled the Key.
 * @note	'Enable alpha channel support in post-processing' must be set to 'Allow through tonemapper' to enabled the Key.
 */
UCLASS(BlueprintType, PrioritizeCategories=(Blackmagic, Output, Synchronization, Debug), meta = (MediaIOCustomLayout = "Blackmagic"))
class BLACKMAGICMEDIAOUTPUT_API UBlackmagicMediaOutput : public UMediaOutput
{
	GENERATED_UCLASS_BODY()

public:
	/** The device, port and video settings that correspond to the output. */
	UPROPERTY(EditAnywhere, Category = "Blackmagic", meta = (DisplayName = "Configuration"))
	FMediaIOOutputConfiguration OutputConfiguration;

public:
	/** Size of the buffer that holds rendered audio samples, a bigger buffer will produce an output of greater quality but will introduce more delay. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	int32 AudioBufferSize = 5*1024;

	/** Sample rate of the audio output. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	EBlackmagicMediaOutputAudioSampleRate AudioSampleRate = EBlackmagicMediaOutputAudioSampleRate::SR_48k;

	/** Number of audio channels to output. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	EBlackmagicMediaAudioOutputChannelCount OutputChannelCount = EBlackmagicMediaAudioOutputChannelCount::CH_2; 
	
	/** Bit depth of each audio sample. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	EBlackmagicMediaOutputAudioBitDepth AudioBitDepth = EBlackmagicMediaOutputAudioBitDepth::Signed_16Bits; 
	
	/** Whether to embed the Engine's timecode to the output frame. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	EMediaIOTimecodeFormat TimecodeFormat;

	/** Native data format internally used by the device before being converted to SDI/HDMI signal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	EBlackmagicMediaOutputPixelFormat PixelFormat;
	
	/** Invert Key Output */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output")
	bool bInvertKeyOutput;

	/** Whether to capture and output audio from the engine. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	bool bOutputAudio;

	/**
	 * Number of frame used to transfer from the system memory to the Blackmagic card.
	 * A smaller number is most likely to cause missed frame.
	 * A bigger number is most likely to increase latency.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output", meta = (UIMin = 3, UIMax = 4, ClampMin = 3, ClampMax = 4))
	int32 NumberOfBlackmagicBuffers;

	/**
	 * Only make sense in interlaced mode.
	 * When creating a new Frame the 2 fields need to have the same timecode value.
	 * The Engine's need a TimecodeProvider (or the default system clock) that is in sync with the generated fields.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output")
	bool bInterlacedFieldsTimecodeNeedToMatch;

	/**
	 * If this is enabled when outputting media in interlace format, frames will be treated as progressive and will be outputted as-is. (Experimental)
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output")
	bool bOutputInterlaceAsProgressive = false;

	/**
	 * Whether to only schedule interlace frames if we land on an even timecode. Useful when you have an interlace input
	 * and want to ensure that we interlace the original frames. (Experimental)
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output")
	bool bOutputInterlaceOnEvenFrames = false;
	
	/**
	 * HDR Metadata of the video signal.
	 */
	UPROPERTY(EditAnywhere, Category = "Output", DisplayName="HDR")
	FBlackmagicMediaHDROptions HDROptions;

	/**
	 * Whether to use multi threaded scheduling which should improve performance when outputting 4k and 8k content. (Experimental)
	 */
	UPROPERTY(BlueprintReadWrite, AdvancedDisplay, EditAnywhere, Category = "Output")
	bool bUseMultithreadedScheduling = false;
	
	/** Try to maintain a the engine "Genlock" with the VSync signal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Synchronization")
	bool bWaitForSyncEvent;

public:

	/** Log a warning when there's a drop frame. */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bLogDropFrame;

	/** Burn Frame Timecode on the output without any frame number clipping. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Debug", meta = (DisplayName = "Burn Frame Timecode"))
	bool bEncodeTimecodeInTexel;

public:
	bool Validate(FString& FailureReason) const;

	FFrameRate GetRequestedFrameRate() const;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;

public:
	//~ UObject interface
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InPropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

};